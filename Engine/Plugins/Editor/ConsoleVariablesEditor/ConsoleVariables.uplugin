{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Console Variables Editor",
	"Description": "Save, load and control Console Variables (cvars) from this panel using Slate.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"EnabledByDefault": true,
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Plugins":
	[
		{
			"Name": "ConcertSyncClient",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		},
		{
			"Name": "ConcertMain",
			"Enabled": true
		},
		{
			"Name": "ConcertSharedSlate",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "ConsoleVariablesEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ConsoleVariablesEditorRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
