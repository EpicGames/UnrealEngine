// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class ConsoleVariablesEditor : ModuleRules
{
	public ConsoleVariablesEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"OutputLog"
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetDefinition",
				"AssetRegistry",
				"AssetTools",
				"CoreUObject",
				"Concert",
				"ConcertSharedSlate",
				"ConcertSyncClient",
				"ConcertSyncCore",
				"ConcertTransport",
				"ConsoleVariablesEditorRuntime",
				"ContentBrowser",
				"Engine",
				"EditorStyle",
				"EditorWidgets",
				"InputCore",
				"Kismet",
				"Projects",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"ToolMenus", 
				"ToolWidgets",
				"UnrealEd",
				"WorkspaceMenuStructure"
			}
		);
	}
}
