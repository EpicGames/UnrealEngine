// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ConsoleVariables.h"

#include "ConsoleVariablesAsset.h"
#include "ConsoleVariablesEditorModule.h"
#include "ConsoleVariablesEditorStyle.h"
#include "ContentBrowserMenuContexts.h"
#include "Editor.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_ConsoleVariables"

FText UAssetDefinition_ConsoleVariables::GetAssetDisplayName() const
{
	return LOCTEXT("ConsoleVariable_AssetName", "Console Variable Collection");
}

TSoftClassPtr<UObject> UAssetDefinition_ConsoleVariables::GetAssetClass() const
{
	return UConsoleVariablesAsset::StaticClass();
}

FLinearColor UAssetDefinition_ConsoleVariables::GetAssetColor() const
{
	return FColor(238, 181, 235, 255);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ConsoleVariables::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Misc, LOCTEXT("ConsoleVariable_CategorySection", "Other"), ECategoryMenuType::Section) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_ConsoleVariables::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	if (Objects.Num() && Objects[0])
	{
		FConsoleVariablesEditorModule& Module = FConsoleVariablesEditorModule::Get();
					
		Module.OpenConsoleVariablesDialogWithAssetSelected(FAssetData(Objects[0]));
	}
	
	return EAssetCommandResult::Handled;
}

FAssetOpenSupport UAssetDefinition_ConsoleVariables::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	FAssetOpenSupport Support(OpenSupportArgs.OpenMethod, OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit);
	Support.RequiredToolkitMode = EToolkitMode::WorldCentric;
	return Support;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_ConsoleVariablesAsset
{
	void ExecuteOpenAssetInEditor(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (Context && Context->SelectedAssets.Num() == 1)
		{
			FConsoleVariablesEditorModule& Module = FConsoleVariablesEditorModule::Get();
			Module.OpenConsoleVariablesDialogWithAssetSelected(Context->SelectedAssets[0]);
		}
	}
	
	bool CanExecuteOpenAssetInEditor(const FToolMenuContext& InContext)
	{
		return UContentBrowserAssetContextMenuContext::GetNumAssetsSelected(InContext) == 1;
	}
	
	void ExecuteVariableCollection(const FToolMenuContext& InContext, bool bOnlyIncludeChecked)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			check(GIsEditor);
			UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
			
			for (const FAssetData& Asset : Context->SelectedAssets)
			{
				if (Asset.GetClass() == UConsoleVariablesAsset::StaticClass())
				{
					if (UConsoleVariablesAsset* ConsoleVariableAsset = Cast<UConsoleVariablesAsset>(Asset.GetAsset()))
					{
						ConsoleVariableAsset->ExecuteSavedCommands(CurrentWorld, bOnlyIncludeChecked);
					}
				}
			}
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, [] {
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UConsoleVariablesAsset::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					InSection.AddMenuEntry("AssetActions_OpenVariableCollection",
						LOCTEXT("AssetActions_OpenVariableCollectionLabel", "Open Variable Collection in Editor"),
						LOCTEXT("AssetActions_OpenVariableCollectionToolTip", "Open this console variable collection in the Console Variables Editor. Select only one asset at a time."),
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "SystemWideCommands.SummonOpenAssetDialog"),
						FToolUIAction(
							FToolMenuExecuteAction::CreateStatic(&ExecuteOpenAssetInEditor),
							FToolMenuCanExecuteAction::CreateStatic(&CanExecuteOpenAssetInEditor),
							FToolMenuGetActionCheckState()
						));
					
					InSection.AddMenuEntry("AssetActions_ExecuteVariableCollection",
						LOCTEXT("AssetActions_ExecuteVariableCollectionLabel", "Execute Variable Collection"),
						LOCTEXT("AssetActions_ExecuteVariableCollectionToolTip", "Executes all commands and variables saved to the selected assets."),
						FSlateIcon(FConsoleVariablesEditorStyle::Get().GetStyleSetName(), "ConsoleVariables.ToolbarButton.Small"),
						FToolUIAction(FToolMenuExecuteAction::CreateStatic(&ExecuteVariableCollection, false)));
					
					InSection.AddMenuEntry("AssetActions_ExecuteOnlyCheckedInVariableCollection",
						LOCTEXT("AssetActions_ExecuteOnlyCheckedInVariableCollectionLabel", "Execute Variable Collection (Only Checked)"),
						LOCTEXT("AssetActions_ExecuteOnlyCheckedInVariableCollectionToolTip", "Executes commands and variables with a Checked checkstate saved to the selected assets. Checkstates can be edited in the Console Variables Editor UI."),
						FSlateIcon(FConsoleVariablesEditorStyle::Get().GetStyleSetName(), "ConsoleVariables.ToolbarButton.Small"),
						FToolUIAction(FToolMenuExecuteAction::CreateStatic(&ExecuteVariableCollection, true)));
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE