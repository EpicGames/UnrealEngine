// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_ConsoleVariables.generated.h"

UCLASS()
class UAssetDefinition_ConsoleVariables : public UAssetDefinitionDefault
{
	GENERATED_BODY()
	
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	// UAssetDefinition End
};
