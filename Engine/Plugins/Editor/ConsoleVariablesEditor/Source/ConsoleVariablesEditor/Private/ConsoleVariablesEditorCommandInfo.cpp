// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleVariablesEditorCommandInfo.h"

#include "ConsoleVariablesEditorLog.h"
#include "ConsoleVariablesEditorModule.h"
#include "MultiUser/ConsoleVariableSync.h"

#include "Algo/Find.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/GameEngine.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

void FConsoleVariablesEditorCommandInfo::ExecuteCommand(
	const FString& NewValueAsString, const bool bShouldTransactInConcert,
	const bool bSetInSession, const bool bIsInteractiveChange)
{
	const FString FullCommand = FString::Printf(TEXT("%s %s"), *Command, *NewValueAsString).TrimStartAndEnd();
	if (IConsoleVariable* AsVariable = GetConsoleVariablePtr())
	{
		AsVariable->Set(*NewValueAsString, GetSource());
		bSetInCurrentSession = bSetInSession;

		if (!bIsInteractiveChange)
		{
			PrintCommandOrVariable();
		}
	}
	else
	{
		GEngine->Exec(GetCurrentWorld(), *FullCommand);
	}
	if (!bIsInteractiveChange && bShouldTransactInConcert)
	{
		FConsoleVariablesEditorModule::Get().SendMultiUserConsoleVariableChange(ERemoteCVarChangeType::Update, Command, NewValueAsString, GetSource());
	}
}

void FConsoleVariablesEditorCommandInfo::PrintCommandOrVariable()
{
	FString FullCommand = FString::Printf(TEXT("%s"), *Command).TrimStartAndEnd();
	
	FString Value;
	if (GetCurrentValueAsString(Value))
	{
		FullCommand = FString::Printf(TEXT("%s %s"), *FullCommand, *Value).TrimStartAndEnd();
	}

	UE_LOGF(LogConsoleVariablesEditor, Log, "Cmd: %ls", *FullCommand);
}

/** Get a reference to the cached console object. May return nullptr if unregistered. */
IConsoleObject* FConsoleVariablesEditorCommandInfo::GetConsoleObjectPtr()
{
	// If the console object ptr null, try to find if not already attempted
	// May return nullptr if unregistered
	if (!ConsoleObjectPtr && !bHasAttemptedFind)
	{
		bHasAttemptedFind = true;

		// Remove additional params, if they exist
		int32 IndexOfSpace = INDEX_NONE;
		if (Command.FindChar(TEXT(' '), IndexOfSpace))
		{
			const FStringView CommandKeyView = FStringView(*Command, IndexOfSpace).TrimStartAndEnd();
			TStringBuilder<260> CommandKey;
			CommandKey.Append(CommandKeyView);
			ConsoleObjectPtr = IConsoleManager::Get().FindConsoleObject(*CommandKey);
		}
		else
		{
			ConsoleObjectPtr = IConsoleManager::Get().FindConsoleObject(*Command);
		}
	}
	return ConsoleObjectPtr;
}

/** Return the console object as a console variable object if applicable. May return nullptr if unregistered. */
IConsoleVariable* FConsoleVariablesEditorCommandInfo::GetConsoleVariablePtr()
{
	if (IConsoleObject* ObjectPtr = GetConsoleObjectPtr())
	{
		return ObjectPtr->AsVariable();
	}
	
	return nullptr;
}

/**
 *Return the console object as a console command object if applicable.
 *Does not consider externally parsed console commands, as they have no associated objects.
 */
IConsoleCommand* FConsoleVariablesEditorCommandInfo::GetConsoleCommandPtr()
{
	if (IConsoleObject* ObjectPtr = GetConsoleObjectPtr())
	{
		return ObjectPtr->AsCommand();
	}
	
	return nullptr;
}

UWorld* FConsoleVariablesEditorCommandInfo::GetCurrentWorld()
{
	UWorld* CurrentWorld = nullptr;
	if (GIsEditor)
	{
		CurrentWorld = GEditor->GetEditorWorldContext().World();
	}
	else if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		CurrentWorld = GameEngine->GetGameWorld();
	}
	return CurrentWorld;
}

FString FConsoleVariablesEditorCommandInfo::GetHelpText()
{
	if (const IConsoleVariable* AsVariable = GetConsoleVariablePtr())
	{
		return FString(AsVariable->GetHelp());
	}
	return "";
}

EConsoleVariableFlags FConsoleVariablesEditorCommandInfo::GetSource()
{
	if (const IConsoleObject* ConsoleObject = GetConsoleObjectPtr())
	{
		return (EConsoleVariableFlags)((uint32)ConsoleObject->GetFlags() & ECVF_SetByMask);
	}
	return ECVF_Default;
}

void FConsoleVariablesEditorCommandInfo::ClearSourceFlags()
{
	if (IConsoleObject* ConsoleObject = GetConsoleObjectPtr())
	{
		for (const FStaticConsoleVariableFlagInfo& StaticConsoleVariableFlagInfo : SupportedFlags)
		{
			ConsoleObject->ClearFlags(StaticConsoleVariableFlagInfo.Flag);
		}
	}
}

void FConsoleVariablesEditorCommandInfo::SetSourceFlag(const EConsoleVariableFlags InSource)
{
	if (IConsoleVariable* AsVariable = GetConsoleVariablePtr())
	{
		AsVariable->Set(*AsVariable->GetString(), StartupSource);
		return;
	}
	
	const uint32 OldPri = (uint32)GetSource();
	const uint32 NewPri = (uint32)InSource;
	if (NewPri < OldPri)
	{
		return;
	}
	
	if (IConsoleObject* ConsoleObject = GetConsoleObjectPtr())
	{
		ClearSourceFlags();
		ConsoleObject->SetFlags((EConsoleVariableFlags)InSource);
	}
}

FText FConsoleVariablesEditorCommandInfo::GetSourceAsText()
{
	// Non-variables don't really have a source
	if (ObjectType != EConsoleObjectType::Variable)
	{
		return LOCTEXT("Source_IsNotConsoleVariableButConsoleCommand", "Command");
	}
	if (bSetInCurrentSession)
	{
		return LOCTEXT("Source_SetByCurrentPreset", "Session");
	}
	
	return ConvertConsoleVariableSetByFlagToText(GetSource());
}

FText FConsoleVariablesEditorCommandInfo::ConvertConsoleVariableSetByFlagToText(const EConsoleVariableFlags InFlag)
{
	FText ReturnValue = LOCTEXT("UnknownSource", "<UNKNOWN>"); 
	if (const FStaticConsoleVariableFlagInfo* Match = Algo::FindByPredicate(
		SupportedFlags,
			[InFlag](const FStaticConsoleVariableFlagInfo& Comparator)
			{
				return Comparator.Flag == InFlag;
			}))
	{
		ReturnValue = (*Match).DisplayText;
	}
	
	return ReturnValue;
}

bool FConsoleVariablesEditorCommandInfo::GetCurrentValueAsString(FString& OutValueAsString)
{
	if (const IConsoleVariable* AsVariable = GetConsoleVariablePtr())
	{
		OutValueAsString = AsVariable->GetString();

		return true;
	}

	return false;
}

bool FConsoleVariablesEditorCommandInfo::IsCurrentValueDifferentFromInputValue(const FString& InValueToCompare)
{
	if (const IConsoleVariable* AsVariable = GetConsoleVariablePtr())
	{
		// Floats sometimes return true erroneously because they can be stringified as e.g '1' or '1.0' by different functions.
		if (AsVariable->IsVariableFloat())
		{
			const float A = AsVariable->GetFloat();
			const float B = FCString::Atof(*InValueToCompare);
			return !FMath::IsNearlyEqual(A, B);
		}
		
		return !AsVariable->GetString().Equals(InValueToCompare);
	}
	else if (ObjectType == EConsoleObjectType::NullObject || ObjectType == EConsoleObjectType::Command)
	{
		return true;
	}
	return false;
}

void FConsoleVariablesEditorCommandInfo::OnConsoleVariableChanged(IConsoleVariable* ChangedVariable)
{
	FConsoleVariablesEditorModule::Get().OnConsoleVariableChanged(*this, ChangedVariable);
}

#undef LOCTEXT_NAMESPACE
