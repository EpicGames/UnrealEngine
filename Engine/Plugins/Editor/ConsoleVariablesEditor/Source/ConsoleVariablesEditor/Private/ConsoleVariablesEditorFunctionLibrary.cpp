// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleVariablesEditorFunctionLibrary.h"

#include "ConsoleVariablesAsset.h"
#include "ConsoleVariablesEditorCommandInfo.h"
#include "ConsoleVariablesEditorLog.h"
#include "ConsoleVariablesEditorModule.h"

#include "MultiUser/ConsoleVariableSyncData.h"

#include "AssetRegistry/AssetData.h"
#include "Widgets/Input/SCheckBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConsoleVariablesEditorFunctionLibrary)

UConsoleVariablesAsset* UConsoleVariablesEditorFunctionLibrary::GetCurrentlyLoadedPreset()
{
	const FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();
	
	return ConsoleVariablesEditorModule.GetPresetAsset();
}

void UConsoleVariablesEditorFunctionLibrary::LoadPresetIntoConsoleVariablesEditor(const UConsoleVariablesAsset* InAsset, const EConsoleVariablesEditorPresetImportMode InImportMode)
{
	if (!InAsset)
	{
		UE_LOGF(LogConsoleVariablesEditor, Error,
			"%s: InAsset was nullptr, please verify that InAsset is valid.",
			__FUNCTION__);
		return;
	}
	
	const FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();

	ConsoleVariablesEditorModule.OpenConsoleVariablesDialogWithAssetSelected(FAssetData(InAsset), InImportMode);
}

bool UConsoleVariablesEditorFunctionLibrary::CopyCurrentListToAsset(UConsoleVariablesAsset* InAsset)
{
	if (!InAsset)
	{
		UE_LOGF(LogConsoleVariablesEditor, Error,
			"%s: InAsset was nullptr, please verify that InAsset is valid.",
			__FUNCTION__);
		return false;
	}
	
	const FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();

	InAsset->CopyFrom(ConsoleVariablesEditorModule.GetPresetAsset());
	InAsset->MarkPackageDirty();
	return true;
}

bool UConsoleVariablesEditorFunctionLibrary::AddValidatedCommandToCurrentPreset(const FString NewCommand)
{
	const FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();

	if (const FReply Reply = ConsoleVariablesEditorModule.ValidateConsoleInputAndAddToCurrentPreset(FText::FromString(NewCommand));
		Reply.IsEventHandled())
	{
		GetCurrentlyLoadedPreset()->MarkPackageDirty();

		return true;
	}

	return false;
}

bool UConsoleVariablesEditorFunctionLibrary::RemoveCommandFromCurrentPreset(const FString NewCommand)
{
	const FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();
	
	if (const bool bSuccess = GetCurrentlyLoadedPreset()->RemoveConsoleVariable(NewCommand))
	{
		ConsoleVariablesEditorModule.RebuildList();
		
		GetCurrentlyLoadedPreset()->MarkPackageDirty();

		return bSuccess;
	}

	return false;
}

bool UConsoleVariablesEditorFunctionLibrary::GetListOfCommandsFromPreset(
 	const UConsoleVariablesAsset* InAsset, TArray<FString>& OutCommandList)
{
	if (!InAsset)
	{
		UE_LOGF(LogConsoleVariablesEditor, Error,
			"%s: InAsset was nullptr, please verify that InAsset is valid.",
			__FUNCTION__);
		return false;
	}

	OutCommandList.Empty();
	for (const FConsoleVariablesEditorAssetSaveData& Command : InAsset->GetSavedCommands())
	{
		OutCommandList.Add(Command.CommandName);
	}

	if (OutCommandList.Num() < 0)
	{
		UE_LOGF(LogConsoleVariablesEditor, Error,
			"%s: InAsset has no saved commands. Please verify you're using the right asset.",
			__FUNCTION__);
		return false;
	}
	
	return true;
}

bool UConsoleVariablesEditorFunctionLibrary::SetConsoleVariableByName_Float(const FString InCommandName,
	const float InValue)
{
	return SetConsoleVariableByName_String(InCommandName, FString::SanitizeFloat(InValue));
}

bool UConsoleVariablesEditorFunctionLibrary::SetConsoleVariableByName_Bool(const FString InCommandName,
	const bool InValue)
{
	return SetConsoleVariableByName_String(InCommandName, InValue ? "true" : "false");
}

bool UConsoleVariablesEditorFunctionLibrary::SetConsoleVariableByName_Int(const FString InCommandName,
	const int32 InValue)
{
	return SetConsoleVariableByName_String(InCommandName, FString::FromInt(InValue));
}

bool UConsoleVariablesEditorFunctionLibrary::SetConsoleVariableByName_String(const FString InCommandName,
	const FString InValue)
{
	FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();

	if (const TWeakPtr<FConsoleVariablesEditorCommandInfo> CommandInfo =
		ConsoleVariablesEditorModule.FindCommandInfoByName(InCommandName); CommandInfo.IsValid())
	{
		if (CommandInfo.Pin()->GetConsoleVariablePtr())
		{
			CommandInfo.Pin()->ExecuteCommand(InValue);
			return true;
		}
		
		UE_LOGF(LogConsoleVariablesEditor, Error,
			"%s: Command %ls is not a variable type. Please do not enter console commands, only console variables.",
			__FUNCTION__, *InCommandName);
	}
	else
	{
		UE_LOGF(LogConsoleVariablesEditor, Error,
			"%s: FConsoleVariablesEditorCommandInfo was not foundwith given name: %ls.",
			__FUNCTION__, *InCommandName);
	}
	return false;
}

bool UConsoleVariablesEditorFunctionLibrary::GetConsoleVariableStringValue(
	const FString InCommandName, FString& OutValue)
{
	FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();

	if (const TWeakPtr<FConsoleVariablesEditorCommandInfo> CommandInfo =
		ConsoleVariablesEditorModule.FindCommandInfoByName(InCommandName); CommandInfo.IsValid())
	{
		if (const IConsoleVariable* AsVariable = CommandInfo.Pin()->GetConsoleVariablePtr())
		{
			OutValue = AsVariable->GetString();
			return true;
		}
	}

	UE_LOGF(LogConsoleVariablesEditor, Error,
		"%s: FConsoleVariablesEditorCommandInfo was not foundwith given name: %ls.",
		__FUNCTION__, *InCommandName);
	return false;
}

bool UConsoleVariablesEditorFunctionLibrary::GetConsoleVariableSourceByName(const FString InCommandName, FString& OutValue)
{
	FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();

	if (const TWeakPtr<FConsoleVariablesEditorCommandInfo> CommandInfo =
		ConsoleVariablesEditorModule.FindCommandInfoByName(InCommandName); CommandInfo.IsValid())
	{
		OutValue = CommandInfo.Pin()->GetSourceAsText().ToString();
		return true;
	}

	UE_LOGF(LogConsoleVariablesEditor, Error,
		"%s: FConsoleVariablesEditorCommandInfo was not found with given name: %ls.",
		__FUNCTION__, *InCommandName);
	return false;
}

bool UConsoleVariablesEditorFunctionLibrary::GetEnableMultiUserCVarSync()
{
	if (const UConcertCVarSynchronization* CVarSync = GetDefault<UConcertCVarSynchronization>())
	{
		return CVarSync->bSyncCVarTransactions;
	}

	UE_LOGF(LogConsoleVariablesEditor, Error,
		"%s: UConcertCVarSynchronization was not found, please verify the Multi-user plugin is enabled and this object class is compiled.",
		__FUNCTION__);
	return false;
}

void UConsoleVariablesEditorFunctionLibrary::SetEnableMultiUserCVarSync(bool bNewSetting)
{
	if (UConcertCVarSynchronization* CVarSync = GetMutableDefault<UConcertCVarSynchronization>())
	{
		CVarSync->bSyncCVarTransactions = bNewSetting;
		return;
	}

	UE_LOGF(LogConsoleVariablesEditor, Error,
		"%s: UConcertCVarSynchronization was not found, please verify the Multi-user plugin is enabled and this object class is compiled.",
		__FUNCTION__);
}
