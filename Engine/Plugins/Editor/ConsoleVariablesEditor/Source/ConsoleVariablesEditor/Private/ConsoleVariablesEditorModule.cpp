// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleVariablesEditorModule.h"

#include "ConsoleVariablesAsset.h"
#include "ConsoleVariablesEditorCommandInfo.h"
#include "ConsoleVariablesEditorLog.h"
#include "ConsoleVariablesEditorProjectSettings.h"
#include "ConsoleVariablesEditorStyle.h"
#include "HAL/IConsoleManager.h"
#include "MultiUser/ConsoleVariableSync.h"
#include "MultiUser/ConsoleVariableSyncData.h"
#include "Views/MainPanel/ConsoleVariablesEditorMainPanel.h"

#include "Algo/AllOf.h"
#include "Algo/Find.h"
#include "Framework/Docking/TabManager.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "ToolMenus.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "FConsoleVariablesEditorModule"

namespace UE::ConsoleVariablesEditor::ModuleUtils::Private
{
	/**
	 * Warn the user once if attempting to sync Cvars in multi-user if in PIE.
	 * @return Whether the user is in PIE.
	 */
	static bool CheckIsRunningPieAndWarnSyncDisabledInPie()
	{
		/* Have we warned the user about PIE in the Console Variables Editor? */
		static bool bHaveWarnedAboutPIE = false;
	
		const bool bRunningInPIE =  GEditor && GEditor->IsPlaySessionInProgress();
		if (bRunningInPIE && !bHaveWarnedAboutPIE)
		{
			UE_LOGF(LogConsoleVariablesEditor, Display, "Play In Editor is about to start or has started; Multi-User Cvar sync is suspended during PIE.");
			bHaveWarnedAboutPIE = bRunningInPIE;
		}
		return bRunningInPIE;
	}
}

const FName FConsoleVariablesEditorModule::ConsoleVariablesToolkitPanelTabId(TEXT("ConsoleVariablesToolkitPanel"));

FConsoleVariablesEditorModule& FConsoleVariablesEditorModule::Get()
{
	static const FName ConsoleVariablesEditorModuleName("ConsoleVariablesEditor");
	return FModuleManager::GetModuleChecked<FConsoleVariablesEditorModule>(ConsoleVariablesEditorModuleName);
}

void FConsoleVariablesEditorModule::StartupModule()
{
	FConsoleVariablesEditorStyle::Initialize();

	FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FConsoleVariablesEditorModule::OnFEngineLoopInitComplete);
}

void FConsoleVariablesEditorModule::PreUnloadCallback()
{
	if (OnConsoleObjectUnregisteredHandle.IsValid())
	{
		IConsoleManager::Get().OnConsoleObjectUnregistered().Remove(OnConsoleObjectUnregisteredHandle);
		OnConsoleObjectUnregisteredHandle.Reset();
	}
}

void FConsoleVariablesEditorModule::ShutdownModule()
{
	UToolMenus::UnregisterOwner(this);

	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);

	FConsoleVariablesEditorStyle::Shutdown();
	
	MainPanel.Reset();

	ConsoleObjectsMainReference.Empty();

	// Unregister project settings
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Console Variables Editor");
	}
}

void FConsoleVariablesEditorModule::SavePreset() const
{
	if (MainPanel.IsValid())
	{
		MainPanel->SaveCurrentPreset();
	}
}

void FConsoleVariablesEditorModule::SaveSpecificPreset(const TObjectPtr<UConsoleVariablesAsset> Preset) const
{
	if (MainPanel.IsValid())
	{
		MainPanel->SaveSpecificPreset(Preset);
	}
}

void FConsoleVariablesEditorModule::SavePresetAs() const
{
	if (MainPanel.IsValid())
	{
		MainPanel->SaveCurrentPresetAs();
	}
}

void FConsoleVariablesEditorModule::SaveSpecificPresetAs(const TObjectPtr<UConsoleVariablesAsset> Preset) const
{
	if (MainPanel.IsValid())
	{
		MainPanel->SaveSpecificPresetAs(Preset);
	}
}

void FConsoleVariablesEditorModule::OpenConsoleVariablesDialogWithPreset(
	const TObjectPtr<UConsoleVariablesAsset> Preset) const
{
	if (MainPanel.IsValid())
	{
		MainPanel->ImportPreset(Preset);
	}
}

void FConsoleVariablesEditorModule::OpenConsoleVariablesDialogWithAssetSelected(const FAssetData& InAssetData, const EConsoleVariablesEditorPresetImportMode InImportMode) const
{
	if (InAssetData.IsValid())
	{
		OpenConsoleVariablesEditor();
	}

	if (MainPanel.IsValid())
	{
		MainPanel->ImportPreset(InAssetData, InImportMode);
	}
}

void FConsoleVariablesEditorModule::QueryAndBeginTrackingConsoleVariables()
{
	const int32 VariableCount = ConsoleObjectsMainReference.Num();
	
	ConsoleObjectsMainReference.Empty(VariableCount);
	
	IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(FConsoleObjectVisitor::CreateLambda(
		[this] (const TCHAR* Key, IConsoleObject* ConsoleObject)
		{
			if (!ConsoleObject || ConsoleObject->TestFlags(ECVF_Unregistered))
			{
				return;
			}

			const TSharedRef<FConsoleVariablesEditorCommandInfo> Info =
				MakeShared<FConsoleVariablesEditorCommandInfo>(Key);
			
			Info->StartupSource = Info->GetSource();
			Info->ConsoleObjectPtr = ConsoleObject;
			Info->bHasAttemptedFind = true;

			if (IConsoleVariable* AsVariable = ConsoleObject->AsVariable())
			{
				Info->OnVariableChangedCallbackHandle = AsVariable->OnChangedDelegate().AddSP(
					Info, &FConsoleVariablesEditorCommandInfo::OnConsoleVariableChanged);
			}
			
			AddConsoleObjectCommandInfoToMainReference(Info);
		}),
		TEXT(""));

	OnConsoleObjectUnregisteredHandle = IConsoleManager::Get().OnConsoleObjectUnregistered().AddRaw(this, &FConsoleVariablesEditorModule::OnConsoleObjectUnregistered);
}

void FConsoleVariablesEditorModule::AddConsoleObjectCommandInfoToMainReference(TSharedRef<FConsoleVariablesEditorCommandInfo> InCommandInfo)
{
	ConsoleObjectsMainReference.Add(InCommandInfo->Command, InCommandInfo);
}

TWeakPtr<FConsoleVariablesEditorCommandInfo> FConsoleVariablesEditorModule::FindCommandInfoByName(const FString& NameToSearch, ESearchCase::Type InSearchCase)
{
	TSharedPtr<FConsoleVariablesEditorCommandInfo>* Match = ConsoleObjectsMainReference.Find(NameToSearch);
	return Match ? *Match : nullptr;
}

TArray<TWeakPtr<FConsoleVariablesEditorCommandInfo>> FConsoleVariablesEditorModule::FindCommandInfosMatchingTokens(
	const TArray<FString>& InTokens, ESearchCase::Type InSearchCase)
{
	TArray<TWeakPtr<FConsoleVariablesEditorCommandInfo>> ReturnValue;
	
	for (const TPair<FString, TSharedPtr<FConsoleVariablesEditorCommandInfo>>& It : ConsoleObjectsMainReference)
	{
		const TSharedPtr<FConsoleVariablesEditorCommandInfo>& CommandInfo = It.Value;
		FString CommandSearchableText = CommandInfo->Command + " " + CommandInfo->GetHelpText() + " " + CommandInfo->GetSourceAsText().ToString();
		// Match any
		for (const FString& Token : InTokens)
		{
			bool bMatchFound = false;
			
			// Match all of these
			const FString SpaceDelimiter = " ";
			TArray<FString> OutSpacedArray;
			if (Token.Contains(SpaceDelimiter) && Token.ParseIntoArray(OutSpacedArray, *SpaceDelimiter, true) > 1)
			{
				bMatchFound = Algo::AllOf(OutSpacedArray, [&CommandSearchableText, InSearchCase](const FString& Comparator)
				{
					return CommandSearchableText.Contains(Comparator, InSearchCase);
				});
			}
			else
			{
				bMatchFound = CommandSearchableText.Contains(Token, InSearchCase);
			}

			if (bMatchFound)
			{
				ReturnValue.Add(CommandInfo);
			}
		}
	}

	return ReturnValue;
}

TObjectPtr<UConsoleVariablesAsset> FConsoleVariablesEditorModule::GetPresetAsset() const
{
	return EditingPresetAsset;
}

TObjectPtr<UConsoleVariablesAsset> FConsoleVariablesEditorModule::GetGlobalSearchAsset() const
{
	return EditingGlobalSearchAsset;
}

FReply FConsoleVariablesEditorModule::ValidateConsoleInputAndAddToCurrentPreset(const FText& CommittedText) const
{
	return MainPanel->ValidateConsoleInputAndAddToCurrentPreset(CommittedText);
}

void FConsoleVariablesEditorModule::RebuildList(const FString InConsoleCommandToScrollTo, bool bShouldCacheValues) const
{
	MainPanel->RebuildList(InConsoleCommandToScrollTo, bShouldCacheValues);
}

void FConsoleVariablesEditorModule::RefreshList() const
{
	MainPanel->RefreshList();
}

void FConsoleVariablesEditorModule::UpdatePresetValuesForSave(TObjectPtr<UConsoleVariablesAsset> InAsset)
{
	MainPanel->UpdatePresetValuesForSave(InAsset);
}

bool FConsoleVariablesEditorModule::PopulateGlobalSearchAssetWithVariablesMatchingTokens(const TArray<FString>& InTokens)
{
	// Remove existing commands
	TArray<FConsoleVariablesEditorAssetSaveData> NullList;
	EditingGlobalSearchAsset->ReplaceSavedCommands(NullList);
	
	for (const TWeakPtr<FConsoleVariablesEditorCommandInfo>& CommandInfo : FindCommandInfosMatchingTokens(InTokens))
	{
		EditingGlobalSearchAsset->AddOrSetConsoleObjectSavedData(
			{
				CommandInfo.Pin()->Command,
				CommandInfo.Pin()->GetConsoleVariablePtr() ?
					CommandInfo.Pin()->GetConsoleVariablePtr()->GetString() : "",
				ECheckBoxState::Checked
			}
		);
	}

	return EditingGlobalSearchAsset->GetSavedCommandsCount() > 0;
}

void FConsoleVariablesEditorModule::SendMultiUserConsoleVariableChange(ERemoteCVarChangeType InChangeType, const FString& InVariableName, const FString& InValueAsString, EConsoleVariableFlags InFlags)
{
	using namespace UE::ConsoleVariablesEditor::ModuleUtils::Private; 
	if (MainPanel->GetMultiUserManager().IsLocalUserInMultiUserSession() && !CheckIsRunningPieAndWarnSyncDisabledInPie())
	{
		MainPanel->GetMultiUserManager().SendConsoleVariableChange(InChangeType, InVariableName, InValueAsString, InFlags);
	}
}

void FConsoleVariablesEditorModule::OnRemoteCvarChanged(ERemoteCVarChangeType InChangeType, const FString InName, const FString InValue, EConsoleVariableFlags InFlags)
{
	using namespace UE::ConsoleVariablesEditor::ModuleUtils::Private; 
	if (MainPanel->GetMultiUserManager().IsLocalUserInMultiUserSession() &&
		!CheckIsRunningPieAndWarnSyncDisabledInPie() &&
		GetDefault<UConcertCVarSynchronization>()->bSyncCVarTransactions)
	{
		FScopeMultiUserReceiveCVar CVarChange(*InName, CommandsReceivedFromMultiUser);

		FConsoleVariablesEditorCommandInfo Info(InName);
		if (IConsoleVariable* AsVariable = Info.GetConsoleVariablePtr())
		{
			AsVariable->Set(*InValue, (EConsoleVariableFlags)(InFlags & ECVF_SetByMask));
		}
		else
		{
			GEngine->Exec(FConsoleVariablesEditorCommandInfo::GetCurrentWorld(),
					  *FString::Printf(TEXT("%s %s"), *InName, *InValue));
		}

		if (InChangeType == ERemoteCVarChangeType::Remove)
		{
			EditingPresetAsset->RemoveConsoleVariable(InName);
		}
		else
		{
			EditingPresetAsset->AddOrSetConsoleObjectSavedData(
			{
				InName,
				InValue,
				ECheckBoxState::Checked
			}
			);
		}

		MainPanel->OnRemoteCvarChanged();
	}
}

void FConsoleVariablesEditorModule::OnFEngineLoopInitComplete()
{
	RegisterMenuItem();
	RegisterProjectSettings();
	QueryAndBeginTrackingConsoleVariables();
	CreateEditingPresets();
	
	MainPanel = MakeShared<FConsoleVariablesEditorMainPanel>();
}

void FConsoleVariablesEditorModule::RegisterMenuItem()
{
	FTabSpawnerEntry& BrowserSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		ConsoleVariablesToolkitPanelTabId,
		FOnSpawnTab::CreateRaw(this, & FConsoleVariablesEditorModule::SpawnMainPanelTab))
			.SetIcon(FSlateIcon(FConsoleVariablesEditorStyle::Get().GetStyleSetName(), "ConsoleVariables.ToolbarButton", "ConsoleVariables.ToolbarButton.Small"))
			.SetDisplayName(LOCTEXT("OpenConsoleVariablesEditorMenuItem", "Console Variables"))
			.SetTooltipText(LOCTEXT("OpenConsoleVariablesEditorTooltip", "Open the Console Variables Editor"))
			.SetMenuType(ETabSpawnerMenuType::Enabled);

	BrowserSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory());
}

void FConsoleVariablesEditorModule::RegisterProjectSettings() const
{
	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	{
		// User Project Settings
		const TSharedPtr<ISettingsSection> ProjectSettingsSectionPtr = SettingsModule.RegisterSettings(
			"Project", "Plugins", "Console Variables Editor",
			LOCTEXT("ConsoleVariablesSettingsCategoryDisplayName", "Console Variables Editor"),
			LOCTEXT("ConsoleVariablesSettingsDescription", "Configure the Console Variables Editor user settings"),
			GetMutableDefault<UConsoleVariablesEditorProjectSettings>());
	}
}

void FConsoleVariablesEditorModule::OnConsoleVariableChanged(FConsoleVariablesEditorCommandInfo& CommandInfo, IConsoleVariable* ChangedVariable)
{
	// Note: We disable tracking during automation tests to prevent FindConsoleObject() related performance warnings.
	if (IsEngineExitRequested() || GIsAutomationTesting)
	{
		return;
	}

	check(EditingPresetAsset);

	{
		const FString& Key = CommandInfo.Command;

		FConsoleVariablesEditorAssetSaveData FoundData;
		bool bIsVariableCurrentlyTracked = EditingPresetAsset->FindSavedDataByCommandString(Key, FoundData, ESearchCase::IgnoreCase);

		const UConsoleVariablesEditorProjectSettings* Settings = GetDefault<UConsoleVariablesEditorProjectSettings>();
		check(Settings);

		if (!bIsVariableCurrentlyTracked)
		{
			// If not yet tracked and we want to track variable changes from outside the dialogue,
			// Check if the changed value differs from the startup value before tracking it
			if (Settings->bAddAllChangedConsoleVariablesToCurrentPreset &&
				!Settings->ChangedConsoleVariableSkipList.Contains(Key) &&
				CommandInfo.IsCurrentValueDifferentFromInputValue(CommandInfo.StartupValueAsString))
			{
				if (MainPanel.IsValid())
				{
					MainPanel->AddConsoleObjectToCurrentPreset(
						Key,
						// If we're not in preset mode then pass empty value
						// This forces the row to get the current value at the time it's generated
						MainPanel->GetEditorListMode() ==
						FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset ?
						ChangedVariable->GetString() : "",
						true
					);

					bIsVariableCurrentlyTracked = true;
				}
			}
		}

		if (MainPanel.IsValid())
		{
			MainPanel->RefreshList();
		}

		// If the variable is already tracked or was just added to tracking, run the following code
		if (bIsVariableCurrentlyTracked)
		{
			if (MainPanel.IsValid())
			{
				MainPanel->UpdateCachedValue(Key, ChangedVariable->GetString());
			}

			/**
			 * Here we check the map of recently received variables from other nodes
			 * If the command is in the map and the value is similar, we won't send the value to other nodes
			 * because we can assume that this value came from another node.
			 * This prevents a feedback loop.
			 */
			if (!CommandsReceivedFromMultiUser.Find(Key))
			{
				SendMultiUserConsoleVariableChange(ERemoteCVarChangeType::Update, Key, ChangedVariable->GetString(), ChangedVariable->GetFlags());
			}
		}
	}
}

void FConsoleVariablesEditorModule::OnConsoleObjectUnregistered(const TCHAR* InName, IConsoleObject* InConsoleObject)
{
	// Note: EditingPresetAsset is not being nulled out when destroyed so accessing EditingPresetAsset on shutdown is crashing
	if (IsEngineExitRequested())
	{
		return;
	}

	if (ensure(InName))
	{
		if (ensure(EditingPresetAsset))
		{
			EditingPresetAsset->RemoveConsoleVariable(InName);
		}
	}

	TSharedPtr<FConsoleVariablesEditorCommandInfo> Found;
	if (ConsoleObjectsMainReference.RemoveAndCopyValue(InName, Found))
	{
		if (Found.IsValid())
		{
			// Null out deleted ConsoleObjectPtr for anything still holding reference to TSharedPtr
			Found->ConsoleObjectPtr = nullptr;
		}

		if (MainPanel.IsValid())
		{
			// TODO: Request list refresh, don't force it now as many objects could be unregistered in one frame during reloadconfig
			MainPanel->RefreshList();
		}
	}
}

TObjectPtr<UConsoleVariablesAsset> FConsoleVariablesEditorModule::AllocateTransientPreset(const FName DesiredName) const
{
	const FString PackageName = "/Temp/ConsoleVariablesEditor/PendingConsoleVariablesPresets";

	UPackage* NewPackage = CreatePackage(*PackageName);
	NewPackage->SetFlags(RF_Transient);
	NewPackage->AddToRoot();

	return NewObject<UConsoleVariablesAsset>(
		NewPackage, DesiredName, RF_Transient | RF_Transactional | RF_Standalone);
}

void FConsoleVariablesEditorModule::CreateEditingPresets()
{
	EditingPresetAsset = AllocateTransientPreset("ConsoleVariablesPreset_PendingPreset");
	
	EditingGlobalSearchAsset = AllocateTransientPreset("ConsoleVariablesPreset_GlobalSearch");
}

TSharedRef<SDockTab> FConsoleVariablesEditorModule::SpawnMainPanelTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab).TabRole(ETabRole::NomadTab);
	DockTab->SetContent(MainPanel->GetOrCreateWidget());
	MainPanel->RebuildList();
			
	return DockTab;
}

void FConsoleVariablesEditorModule::OpenConsoleVariablesEditor()
{
	FGlobalTabmanager::Get()->TryInvokeTab(ConsoleVariablesToolkitPanelTabId);
}

void FConsoleVariablesEditorModule::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(EditingPresetAsset);
	Collector.AddReferencedObject(EditingGlobalSearchAsset);
}

FString FConsoleVariablesEditorModule::GetReferencerName() const
{
	return TEXT("FConsoleVariablesEditorModule");
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FConsoleVariablesEditorModule, ConsoleVariablesEditor)
