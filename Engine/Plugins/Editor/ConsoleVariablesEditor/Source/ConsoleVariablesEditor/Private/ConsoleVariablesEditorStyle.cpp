// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleVariablesEditorStyle.h"

#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

TSharedPtr<FSlateStyleSet> FConsoleVariablesEditorStyle::StyleInstance = nullptr;

void FConsoleVariablesEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FConsoleVariablesEditorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

void FConsoleVariablesEditorStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FConsoleVariablesEditorStyle::Get()
{
	return *StyleInstance;
}

const FName& FConsoleVariablesEditorStyle::GetStyleSetName() const
{
	static FName ConsoleVariablesStyleSetName(TEXT("ConsoleVariablesEditor"));
	return ConsoleVariablesStyleSetName;
}

const FSlateBrush* FConsoleVariablesEditorStyle::GetBrush(const FName PropertyName, const ANSICHAR* Specifier, const ISlateStyle* RequestingStyle) const
{
	return StyleInstance->GetBrush(PropertyName, Specifier);
}

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BORDER_BRUSH( RelativePath, ... ) FSlateBorderBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( Style->RootToContentDir(RelativePath, TEXT(".svg") ), __VA_ARGS__)
#define IMAGE_PLUGIN_BRUSH_SVG( PluginName, RelativePath, ... ) FSlateVectorImageBrush( FConsoleVariablesEditorStyle::GetExternalPluginContent(PluginName, RelativePath, ".svg"), __VA_ARGS__)

const FVector2D Icon64x64(64.f, 64.f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon12x12(12.0f, 12.0f);
const FVector2D Icon8x8(8.f, 8.f);

FString FConsoleVariablesEditorStyle::GetExternalPluginContent(const FString& PluginName, const FString& RelativePath, const ANSICHAR* Extension)
{
	FString ContentDir = IPluginManager::Get().FindPlugin(PluginName)->GetBaseDir() / RelativePath + Extension;
	return ContentDir;
}

TSharedRef< FSlateStyleSet > FConsoleVariablesEditorStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShared<FSlateStyleSet>("ConsoleVariablesEditor");

	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("ConsoleVariables"));
	check(Plugin.IsValid());
	if (Plugin.IsValid())
	{
		Style->SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources")));
	}

	Style->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Icons
	Style->Set("ConsoleVariables.ToolbarButton", new IMAGE_BRUSH_SVG("Icons/ConsoleVariables", Icon40x40));
	Style->Set("ConsoleVariables.ToolbarButton.Small", new IMAGE_BRUSH_SVG("Icons/ConsoleVariables", Icon20x20));
	
	Style->Set("ConsoleVariables.Favorite.Outline", new IMAGE_BRUSH_SVG("Icons/FavoriteOutline", Icon40x40));
	Style->Set("ConsoleVariables.Favorite.Outline.Small", new IMAGE_BRUSH_SVG("Icons/FavoriteOutline", Icon16x16));
	
	Style->Set("ConsoleVariables.GlobalSearch", new IMAGE_BRUSH_SVG("Icons/SearchGlobal", Icon40x40));
	Style->Set("ConsoleVariables.GlobalSearch.Small", new IMAGE_BRUSH_SVG("Icons/SearchGlobal", Icon16x16));

	// Brush
	Style->Set("ConsoleVariablesEditor.GroupBorder", new BOX_BRUSH("Common/DarkGroupBorder", FMargin(4.0f / 16.0f)));
	Style->Set("ConsoleVariablesEditor.BrightBorder", new FSlateColorBrush(FStyleColors::Header));

	// Border colors for Results view
	Style->Set("ConsoleVariablesEditor.HeaderRowBorder", new FSlateColorBrush(FStyleColors::Black));
	Style->Set("ConsoleVariablesEditor.CommandGroupBorder", new BOX_BRUSH("Common/DarkGroupBorder", FMargin(4.0f / 16.0f)));
	Style->Set("ConsoleVariablesEditor.DefaultBorder", new FSlateColorBrush(FStyleColors::Transparent));

	// External plugin icons
	// Multi-user Tab/Menu icons
	Style->Set("Icons.MultiUser", new IMAGE_PLUGIN_BRUSH_SVG("ConcertSharedSlate","Content/Icons/icon_MultiUser", Icon16x16));
	
	return Style;
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH
