// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FConsoleVariablesEditorStyle : FSlateStyleSet
{
public:

	static void Initialize();

	static void Shutdown();

	static void ReloadTextures();

	static const ISlateStyle& Get();

	virtual const FName& GetStyleSetName() const override;

	virtual const FSlateBrush* GetBrush(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const ISlateStyle* RequestingStyle = nullptr) const override;

	template< typename WidgetStyleType >
	static const WidgetStyleType& GetWidgetStyle(FName PropertyName, const ANSICHAR* Specifier = nullptr)
	{
		return StyleInstance->GetWidgetStyle<WidgetStyleType>(PropertyName, Specifier);
	}

private:

	static FString GetExternalPluginContent(const FString& PluginName, const FString& RelativePath, const ANSICHAR* Extension);
	
	static TSharedRef<FSlateStyleSet> Create();

	static TSharedPtr<FSlateStyleSet> StyleInstance;
};
