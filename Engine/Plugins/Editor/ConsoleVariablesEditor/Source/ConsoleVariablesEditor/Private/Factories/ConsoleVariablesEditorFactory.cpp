// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/ConsoleVariablesEditorFactory.h"

#include "AssetToolsModule.h"

#include "ConsoleVariablesAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConsoleVariablesEditorFactory)

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditorFactory"

UConsoleVariablesEditorFactory::UConsoleVariablesEditorFactory()
{
	SupportedClass = UConsoleVariablesAsset::StaticClass();

	// This factory manufacture new objects from scratch.
	bCreateNew = true;

	// This factory will open the editor for each new object.
	bEditAfterNew = true;
}

UObject* UConsoleVariablesEditorFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UConsoleVariablesAsset>(InParent, InClass, InName, Flags);
}

bool UConsoleVariablesEditorFactory::ShouldShowInNewMenu() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
