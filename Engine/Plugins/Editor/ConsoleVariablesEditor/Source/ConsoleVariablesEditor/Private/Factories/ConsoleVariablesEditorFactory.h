// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"

#include "ConsoleVariablesEditorFactory.generated.h"

UCLASS(MinimalAPI)
class UConsoleVariablesEditorFactory : public UFactory
{
	GENERATED_BODY()

public:

	UConsoleVariablesEditorFactory();
	
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
};