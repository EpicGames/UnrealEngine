// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertConsoleVariableSessionCustomization.h"

#include "ConsoleVariablesEditorStyle.h"
#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "Framework/Views/ITypedTableView.h"
#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "IConcertSession.h"
#include "IConcertSessionHandler.h"
#include "IConcertSyncClientModule.h"

#include "Misc/AssertionMacros.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/Script.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SCheckBox.h"
#include "EditorFontGlyphs.h"
#include "Styling/AppStyle.h"
#include "DetailWidgetRow.h"


#include "ConcertMessages.h"

#include "MultiUser/ConsoleVariableSyncData.h"

#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"

#define LOCTEXT_NAMESPACE "ConcertConsoleVariableSessionCustomization"

namespace ConsoleVariableSessionUI
{
	static const FName DisplayNameColumnName(TEXT("UserDisplayName"));
	static const FName TransactColumnName(TEXT("Transact"));
}

class SConsoleVariableSessionRow : public SMultiColumnTableRow<TSharedPtr<FConcertCVarDetails>>
{
	SLATE_BEGIN_ARGS(SConsoleVariableSessionRow) {}
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs, TWeakPtr<IConcertSyncClient> InSyncClient,
				   TWeakPtr<IConcertClientSession> InClientSession,
				   TSharedPtr<FConcertCVarDetails> InClientInfo, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		SyncClient = MoveTemp(InSyncClient);
		ClientSession = MoveTemp(InClientSession);
		ClientSetting = MoveTemp(InClientInfo);
		SMultiColumnTableRow<TSharedPtr<FConcertCVarDetails>>::Construct(
			FSuperRowType::FArguments()
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow")),
			InOwnerTableView
		);

		SetToolTipText(MakeAttributeSP(this, &SConsoleVariableSessionRow::GetRowToolTip));
	}

public:
	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		using ThisType = SConsoleVariableSessionRow;
		if (ColumnName == ConsoleVariableSessionUI::DisplayNameColumnName)
		{
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				[
					SNew(STextBlock)
					.Font(this, &SConsoleVariableSessionRow::GetAvatarFont)
					.ColorAndOpacity(this, &SConsoleVariableSessionRow::GetAvatarColor)
					.Text(FEditorFontGlyphs::Square)
				]

				// The client display name.
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.ColorAndOpacity(FLinearColor(0.75f, 0.75f, 0.75f))
					.Padding(FMargin(6.0f, 4.0f))
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("BoldFont"))
						.Text(GetDisplayName())
					]
				];
		}
		else
		{
			check(ColumnName == ConsoleVariableSessionUI::TransactColumnName);

			return  SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				[
					SAssignNew(Transact,SCheckBox)
					.IsChecked(this, &ThisType::GetTransactState)
					.OnCheckStateChanged(this, &ThisType::SetTransactState)
					.IsEnabled(this, &ThisType::CheckBoxEnabled)
					.ToolTipText_Lambda([this](){return GetTransactToolTip();})
				];
		}
	}

	FText GetToolTipNoSync()
	{
		return FText::Format(LOCTEXT("NoSyncEnabled_ToolTip", "Synchronize Disabled on {0}"), GetDisplayName());
	}


	FText GetTransactToolTip()
	{
		TSharedPtr<FConcertCVarDetails> SettingPin = ClientSetting.Pin();
		if(SettingPin.IsValid())
		{
			if (SettingPin->bCVarSyncEnabled)
			{
				return LOCTEXT("Transact_ToolTip", "Sets whether this client should receive CVar changes.");
			}
		}
		return GetToolTipNoSync();
	}

	ECheckBoxState GetTransactState() const
	{
		TSharedPtr<FConcertCVarDetails> SettingPin = ClientSetting.Pin();
		if(SettingPin.IsValid())
		{
			if (SettingPin->Settings.bReceiveCVarChanges)
			{
				return ECheckBoxState::Checked;
			}
		}
		return ECheckBoxState::Unchecked;
	}

	void SetTransactState(ECheckBoxState InValue)
	{
		TSharedPtr<FConcertCVarDetails> SettingPin = ClientSetting.Pin();
		if(SettingPin.IsValid())
		{
			SettingPin->Settings.bReceiveCVarChanges = ECheckBoxState::Checked == InValue?true:false;
			NotifyChange(SettingPin);
		}
	}

	bool CheckBoxEnabled() const
	{
		TSharedPtr<FConcertCVarDetails> Setting = ClientSetting.Pin();
		if (Setting.IsValid())
		{
			return Setting->bCVarSyncEnabled;
		}
		return false;
	}

	void UpdateUI()
	{
		TSharedPtr<FConcertCVarDetails> Setting = ClientSetting.Pin();
		if (Setting.IsValid())
		{
			if (Transact.IsValid())
			{
				Transact->SetEnabled(Setting->bCVarSyncEnabled);
				Transact->SetIsChecked(Setting->Settings.bReceiveCVarChanges);
				Transact->SetToolTipText(GetTransactToolTip());
			}
		}
	}

	FText GetRowToolTip() const
	{
		// This is a tooltip for the entire row. Like display name, the tooltip will not update in real time if the user change its
		// settings. See GetDisplayName() for more info.
		TSharedPtr<FConcertCVarDetails> SettingPin = ClientSetting.Pin();
		return SettingPin.IsValid() ? SettingPin->Details.ToDisplayString() : FText();
	}

	FText GetDisplayName() const
	{
		TSharedPtr<FConcertCVarDetails> SettingPin = ClientSetting.Pin();
		if (SettingPin.IsValid())
		{
			// NOTE: The display name doesn't update in real time at the moment because the concert setting are not propagated
			//       until the client disconnect/reconnect. Since those settings should not change often, this should not
			//       be a major deal breaker for the users.
			TSharedPtr<IConcertClientSession> ClientSessionPin = ClientSession.Pin();
			if (ClientSessionPin.IsValid() && SettingPin->Details.ClientEndpointId == ClientSessionPin->GetSessionClientEndpointId())
			{
				return FText::Format(LOCTEXT("ClientDisplayNameIsYouFmt", "{0} (You)"),
									 FText::FromString(ClientSessionPin->GetLocalClientInfo().DisplayName));
			}

			// Return the ClientInfo cached.
			return FText::FromString(SettingPin->Details.ClientInfo.DisplayName);
		}

		return FText();
	}

	FSlateFontInfo GetAvatarFont() const
	{
		static const FName ButtonIconSyle = TEXT("FontAwesome.10");
		// This font is used to render a small square box filled with the avatar color.
		FSlateFontInfo ClientIconFontInfo = FAppStyle::Get().GetFontStyle(ButtonIconSyle);
		ClientIconFontInfo.Size = 8;
		ClientIconFontInfo.OutlineSettings.OutlineSize = 1;

		TSharedPtr<FConcertCVarDetails> ClientSettingPin = ClientSetting.Pin();
		if (ClientSettingPin.IsValid())
		{
			FConcertSessionClientInfo& Client = ClientSettingPin->Details;
			FLinearColor ClientOutlineColor = Client.ClientInfo.AvatarColor * 0.6f; // Make the font outline darker.
			ClientOutlineColor.A = Client.ClientInfo.AvatarColor.A; // Put back the original alpha.
			ClientIconFontInfo.OutlineSettings.OutlineColor = ClientOutlineColor;
		}
		else
		{
			ClientIconFontInfo.OutlineSettings.OutlineColor = FLinearColor(0.75, 0.75, 0.75); // This is an arbitrary color.
		}

		return ClientIconFontInfo;
	}

	FSlateColor GetAvatarColor() const
	{
		TSharedPtr<FConcertCVarDetails> ClientSettingPin = ClientSetting.Pin();
		if (ClientSettingPin.IsValid())
		{
			return ClientSettingPin->Details.ClientInfo.AvatarColor;
		}

		return FSlateColor(FLinearColor(0.75, 0.75, 0.75)); // This is an arbitrary color.
	}

	void NotifyChange(TSharedPtr<FConcertCVarDetails>& SettingPin)
	{
		check(SettingPin.IsValid());
		const FConcertCVarDetails& Ref = *SettingPin.Get();
		SettingChangeDelegate.Broadcast(Ref);
	}

	FOnConcertCVarDetailChanged& SettingChanged()
	{
		return SettingChangeDelegate;
	}

private:
	TSharedPtr<SCheckBox>		 RecordOnClient;
	TSharedPtr<SCheckBox>		 Transact;

	TWeakPtr<IConcertSyncClient> SyncClient;
	TWeakPtr<IConcertClientSession> ClientSession;
	TWeakPtr<FConcertCVarDetails> ClientSetting;
	FOnConcertCVarDetailChanged SettingChangeDelegate;
};


FConcertCVarDetails  GetLocalSessionConcertClientAsSetting(
	UConcertCVarConfig const* Config,
	TSharedPtr<IConcertClientSession>& Session)
{
	UConcertCVarSynchronization const* Sync = GetDefault<UConcertCVarSynchronization>();
	const FConcertClientInfo &ClientInfo = Session->GetLocalClientInfo();
	FConcertCVarDetails LocalClientSetting;
	LocalClientSetting.Details.ClientEndpointId =
		Session->GetSessionClientEndpointId();
	LocalClientSetting.Details.ClientInfo = ClientInfo;
	LocalClientSetting.Settings = Config->LocalSettings;
	LocalClientSetting.bCVarSyncEnabled = Sync->bSyncCVarTransactions;
	return MoveTemp(LocalClientSetting);
}

void FConcertConsoleVariableSessionCustomization::PopulateClientList()
{
	Clients.Reset();

	if (!IConcertSyncClientModule::IsAvailable())
	{
		return;
	}

	TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
	IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
	TSharedPtr<IConcertClientSession> Session = ConcertClient->GetCurrentSession();

	if (Session)
	{
		UConcertCVarConfig const* Settings = GetDefault<UConcertCVarConfig>();

		Clients.Reserve(Settings->RemoteDetails.Num()+1);

		FConcertCVarDetails Local = GetLocalSessionConcertClientAsSetting(Settings,Session);

		Clients.Emplace(MakeShared<FConcertCVarDetails>(MoveTemp(Local)));

		for (int32 DuplicateIndex = 0; DuplicateIndex < 5; ++DuplicateIndex)
		{
			auto DupeCliant = MakeShared<FConcertCVarDetails>(Local);
			DupeCliant->Details.ClientInfo.AvatarColor = FLinearColor::MakeRandomSeededColor(DuplicateIndex);
			DupeCliant->Details.ClientEndpointId = FGuid::NewGuid();
			DupeCliant->Details.ClientInfo.DisplayName = DupeCliant->Details.ClientEndpointId.ToString();

			Clients.Emplace(DupeCliant);
		}

		for (FConcertCVarDetails const& Client : Settings->RemoteDetails)
		{
			Clients.Emplace(MakeShared<FConcertCVarDetails>(Client));
		}
	}

	if (ClientsListViewWeak.IsValid())
	{
		TSharedPtr<SListViewCVarDetail> ClientsListView = ClientsListViewWeak.Pin();
		ClientsListView->RequestListRefresh();
	}
}

void FConcertConsoleVariableSessionCustomization::UpdateClientSettings(
	EConcertClientStatus Status, const FConcertCVarDetails& InSetting)
{
	int32 Index = Clients.IndexOfByPredicate([&InSetting](const TSharedPtr<FConcertCVarDetails>& Setting) {
		return Setting->Details.ClientEndpointId == InSetting.Details.ClientEndpointId;
	});

	switch(Status)
	{
	case EConcertClientStatus::Connected:
		Clients.Emplace(MakeShared<FConcertCVarDetails>(InSetting));
		break;
	case EConcertClientStatus::Updated:
		check(Index != INDEX_NONE);
		Clients[Index]->Settings = InSetting.Settings;
		Clients[Index]->bCVarSyncEnabled = InSetting.bCVarSyncEnabled;
		break;
	case EConcertClientStatus::Disconnected:
		check(Index != INDEX_NONE);
		Clients.RemoveAt(Index);
		break;
	};

	if (ClientsListViewWeak.IsValid())
	{
		TSharedPtr<SListViewCVarDetail> ClientsListView = ClientsListViewWeak.Pin();
		if (Status == EConcertClientStatus::Disconnected
			|| Status == EConcertClientStatus::Connected )
		{
			ClientsListView->RequestListRefresh();
		}
		else
		{
			TSharedPtr<SConsoleVariableSessionRow> Row = StaticCastSharedPtr<SConsoleVariableSessionRow>(ClientsListView->WidgetFromItem(Clients[Index]));
			if (Row.IsValid())
			{
				Row->UpdateUI();
			}
		}
	}
}

void FConcertConsoleVariableSessionCustomization::SettingChange(const FConcertCVarDetails &Setting)
{
	DetailDelegate.Broadcast(Setting);
}

void FConcertConsoleVariableSessionCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& Settings = DetailLayout.EditCategory(TEXT("MultiUserClientTransaction"));
	Settings.SetDisplayName(LOCTEXT("MultiUserClientTransaction", "Multi-User Client Transaction"));

	TSharedPtr<IPropertyHandle> SettingsProperty = DetailLayout.GetProperty(
		GET_MEMBER_NAME_CHECKED(UConcertCVarConfig, LocalSettings));
	SettingsProperty->MarkHiddenByCustomization();


	TSharedPtr<IPropertyHandle> RemoteSettingsProperty = DetailLayout.GetProperty(
		GET_MEMBER_NAME_CHECKED(UConcertCVarConfig, RemoteDetails));
	RemoteSettingsProperty->MarkHiddenByCustomization();

	PopulateClientList();
	{
		FDetailWidgetRow& Row = Settings.AddCustomRow(LOCTEXT("MultiUserSettings", "Multi-User Client Settings"));
		auto HandleGenerateRow = [this](TSharedPtr<FConcertCVarDetails> InClientInfo,
									const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
		{
			if (!IConcertSyncClientModule::IsAvailable())
			{
				return SNew(STableRow<TSharedPtr<FConcertCVarDetails>>, OwnerTable);
			}

			TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
			TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession();
			TSharedRef<SConsoleVariableSessionRow> Row = SNew(SConsoleVariableSessionRow,
															  ConcertSyncClient,
															  ConcertClientSession,
															  InClientInfo, OwnerTable);
			Row->SettingChanged().AddSP(this,&FConcertConsoleVariableSessionCustomization::SettingChange);
			return Row;
		};
		Row.WholeRowContent()
		[
			SAssignNew(ClientsListViewWeak,SListView<TSharedPtr<FConcertCVarDetails>>)
			.SelectionMode(ESelectionMode::None)
			.ListItemsSource(&Clients)
			.OnGenerateRow_Lambda(HandleGenerateRow)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+SHeaderRow::Column(ConsoleVariableSessionUI::DisplayNameColumnName)
				.DefaultLabel(LOCTEXT("UserDisplayName", "Display Name"))
				+SHeaderRow::Column(ConsoleVariableSessionUI::TransactColumnName)
				.DefaultLabel(LOCTEXT("Transact", "Transact Console Variables"))
			)
		];
	}
};

#undef LOCTEXT_NAMESPACE
