// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessages.h"
#include "IConcertSession.h"
#include "IDetailCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "ConsoleVariableSyncData.h"
#include "IPropertyTypeCustomization.h"

#include "Templates/SharedPointer.h"
#include "Widgets/Views/SListView.h"

DECLARE_MULTICAST_DELEGATE_OneParam(FOnConcertCVarDetailChanged, const FConcertCVarDetails&);

using SListViewCVarDetail = SListView<TSharedPtr<FConcertCVarDetails>>;

class FConcertConsoleVariableSessionCustomization : public IDetailCustomization
{
public:
	void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

	void UpdateClientSettings(EConcertClientStatus Status, const FConcertCVarDetails& RecordSetting);
	void PopulateClientList();

	FOnConcertCVarDetailChanged& OnSettingChanged()
	{
		return DetailDelegate;
	}

private:
	void SettingChange(const FConcertCVarDetails &Setting);

	FOnConcertCVarDetailChanged DetailDelegate;

	TArray<TSharedPtr<FConcertCVarDetails>> Clients;
	TWeakPtr<SListViewCVarDetail>		    ClientsListViewWeak;
};
