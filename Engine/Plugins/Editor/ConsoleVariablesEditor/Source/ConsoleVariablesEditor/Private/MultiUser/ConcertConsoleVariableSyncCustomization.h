// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IDetailCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "MultiUser/ConsoleVariableSyncData.h"
#include "IPropertyTypeCustomization.h"
#include "Misc/AssertionMacros.h"
#include "Types/SlateEnums.h"
#include "UObject/Script.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SHyperlink.h"
#include "DetailWidgetRow.h"

#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"

#define LOCTEXT_NAMESPACE "ConcertConsoleVariableSyncCustomization"

DECLARE_MULTICAST_DELEGATE_OneParam(FOnCVarSyncPropertyValueChanged, bool);

class FConcertConsoleVariableSyncCustomization : public IDetailCustomization
{
public:
	using ThisType = FConcertConsoleVariableSyncCustomization;
	static void OpenMultiUserExcludeProperty()
	{
		FModuleManager::GetModulePtr<ISettingsModule>("Settings")->ShowViewer("Project", "Plugins", "Concert Sync");
	}

	static FOnCVarSyncPropertyValueChanged& OnSyncPropertyValueChanged()
	{
		static FOnCVarSyncPropertyValueChanged SyncPropertyChanged;
		return SyncPropertyChanged;
	}

	void SyncPropertyChanged()
	{
		UConcertCVarSynchronization const* Sync = GetDefault<UConcertCVarSynchronization>();
		OnSyncPropertyValueChanged().Broadcast(Sync->bSyncCVarTransactions);
	}

	void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override
	{
		IDetailCategoryBuilder& TakeSync = DetailLayout.EditCategory("MultiUserSynchronization");
		TakeSync.SetDisplayName(LOCTEXT("MultiUserSynchronization", "Multi-User Synchronization"));

		TSharedPtr<IPropertyHandle> SyncTakeRecordingProperty = DetailLayout.GetProperty(
			GET_MEMBER_NAME_CHECKED(UConcertCVarSynchronization, bSyncCVarTransactions));
		SyncTakeRecordingProperty->MarkHiddenByCustomization();
		SyncTakeRecordingProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &ThisType::SyncPropertyChanged));
		{
			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;

			IDetailPropertyRow& SyncDisplayProperty = TakeSync.AddProperty(SyncTakeRecordingProperty);
			SyncDisplayProperty.GetDefaultWidgets(NameWidget,ValueWidget);
			FDetailWidgetRow& Row = SyncDisplayProperty.CustomWidget(true);

			Row.NameContent()
			[
				NameWidget.ToSharedRef()
			];

			Row.ValueContent()
			.MinDesiredWidth(250)
			.MaxDesiredWidth(0)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				.HAlign(HAlign_Left)
				[
					SNew(SBox)
					.WidthOverride(250.f)
					[
						ValueWidget.ToSharedRef()
					]
				]
			];
		}
	}
};

#undef LOCTEXT_NAMESPACE
