// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUser/ConsoleVariableSync.h"

#include "ConcertTransportMessages.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/IConsoleManager.h"

#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "IConcertSession.h"
#include "IConcertSessionHandler.h"
#include "IConcertSyncClientModule.h"

#include "MultiUser/ConsoleVariableSyncData.h"
#include "MultiUser/ConcertConsoleVariableSessionCustomization.h"
#include "MultiUser/ConcertConsoleVariableSyncCustomization.h"


#include "PropertyEditorModule.h"
#include "Templates/UniquePtr.h"

#include "ConsoleVariablesEditorLog.h"

namespace UE::ConsoleVariables::MultiUser::Private
{

struct FManagerImpl
{
	FManagerImpl()
	{
		if (IConcertSyncClientModule::IsAvailable())
		{
			if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
			{
				IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
				ConcertClient->OnSessionStartup().AddRaw(this, &FManagerImpl::Register);
				ConcertClient->OnSessionShutdown().AddRaw(this, &FManagerImpl::Unregister);

				if (TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession())
				{
					Register(ConcertClientSession.ToSharedRef());
				}

				ConcertClient->OnSessionConnectionChanged().AddRaw(this, &FManagerImpl::OnSessionConnectionChanged);

				RegisterExtensions();

				bIsInitialized = true;
			}
		}
	}

	~FManagerImpl()
	{
		if (IConcertSyncClientModule::IsAvailable())
		{
			Unregister();
			if (const TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
			{
				IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
				ConcertClient->OnSessionStartup().RemoveAll(this);
				ConcertClient->OnSessionShutdown().RemoveAll(this);
			}
		}

		UnregisterExtensions();
	}

	void RegisterExtensions()
	{
		if (GIsEditor)
		{
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

			SyncDelegate = FConcertConsoleVariableSyncCustomization::OnSyncPropertyValueChanged().AddRaw(
				this,&FManagerImpl::OnSyncPropertyChange);

			PropertyEditorModule.RegisterCustomClassLayout(
				UConcertCVarSynchronization::StaticClass()->GetFName(),
				FOnGetDetailCustomizationInstance::CreateLambda(&MakeShared<FConcertConsoleVariableSyncCustomization>));

			PropertyEditorModule.RegisterCustomClassLayout(
				UConcertCVarConfig::StaticClass()->GetFName(),
				FOnGetDetailCustomizationInstance::CreateLambda([this]() {
					Customization = MakeShared<FConcertConsoleVariableSessionCustomization>();
					Customization->OnSettingChanged().AddRaw(this,&FManagerImpl::SettingChange);
					return Customization->AsShared();
			}));
	}
	}

	void UnregisterExtensions()
	{
		FPropertyEditorModule* PropertyEditorModule = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyEditorModule)
		{
			if (UObjectInitialized())
			{
				PropertyEditorModule->UnregisterCustomClassLayout(UConcertCVarSynchronization::StaticClass()->GetFName());
				PropertyEditorModule->UnregisterCustomClassLayout(UConcertCVarConfig::StaticClass()->GetFName());
			}
			FConcertConsoleVariableSyncCustomization::OnSyncPropertyValueChanged().Remove(SyncDelegate);
		}
	}

	void OnSyncPropertyChange(bool Value)
	{
		TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
		if (Session.IsValid())
		{
			FConcertCVarSyncChangeEvent OutEvent;
			OutEvent.EndpointId = Session->GetSessionClientEndpointId();
			OutEvent.bSyncCVarChanges = Value;
			Session->SendCustomEvent(OutEvent, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);

			if (Customization.IsValid())
			{
				UConcertCVarConfig const* Config = GetDefault<UConcertCVarConfig>();
				FConcertCVarDetails Item;
				Item.Details.ClientEndpointId = Session->GetSessionClientEndpointId();
				Item.Settings = Config->LocalSettings;
				Item.bCVarSyncEnabled = Value;
				Customization->UpdateClientSettings(EConcertClientStatus::Updated, Item);
			}
		}
		UConcertCVarSynchronization* Sync = GetMutableDefault<UConcertCVarSynchronization>();
		Sync->SaveConfig();
	}

	void ConnectToSession(IConcertClientSession& InSession)
	{
		UE_LOGF(LogConsoleVariablesEditor, VeryVerbose, "Multi-user Console Variable Editor listening to session: %ls", *InSession.GetSessionInfo().SessionName);

		ClientChangeDelegate = InSession.OnSessionClientChanged().AddRaw(this, &FManagerImpl::OnSessionClientChanged);

		SendInitialState(InSession);
		UpdateSessionClientList();
	}

	void SendInitialState(IConcertClientSession& InSession)
	{
		/** Send Mult-user Console Variable configuration for this client. */
		UConcertCVarConfig const* CurrentSettings = GetDefault<UConcertCVarConfig>();
		FConcertCVarChangeEvent OutEvent;
		OutEvent.EndpointId = InSession.GetSessionClientEndpointId();
		OutEvent.Settings = CurrentSettings->LocalSettings;
		InSession.SendCustomEvent(OutEvent, InSession.GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);

		/** Send the synchronization state for this client.*/
		UConcertCVarSynchronization const* SyncState = GetDefault<UConcertCVarSynchronization>();
		FConcertCVarSyncChangeEvent SyncEvent;
		SyncEvent.EndpointId = OutEvent.EndpointId;
		SyncEvent.bSyncCVarChanges = SyncState->bSyncCVarTransactions;
		InSession.SendCustomEvent(SyncEvent, InSession.GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
	}

	void OnSessionClientChanged(IConcertClientSession& InSession, EConcertClientStatus ClientStatus, const FConcertSessionClientInfo& InClientInfo)
	{
		check(WeakSession.IsValid());

		UConcertCVarConfig* Settings = GetMutableDefault<UConcertCVarConfig>();
		TArray<FConcertCVarDetails>& Clients = Settings->RemoteDetails;

		int32 Index = Clients.IndexOfByPredicate([&InClientInfo](const FConcertCVarDetails& Setting) {
			return Setting.Details.ClientEndpointId == InClientInfo.ClientEndpointId;
		});

		switch(ClientStatus)
		{
		case EConcertClientStatus::Connected:
		{
			check(Index == INDEX_NONE);
			FConcertCVarDetails ConcertCVar;
			ConcertCVar.Details = InClientInfo;
			Clients.Emplace(MoveTemp(ConcertCVar));

			SendInitialState(InSession);
		}
		break;
		case EConcertClientStatus::Disconnected:
			break;
		case EConcertClientStatus::Updated:
			break;
		};

		if (Customization.IsValid())
		{
			Customization->UpdateClientSettings(ClientStatus, Index == INDEX_NONE ? Clients.Last() : Clients[Index] );
		}

		if (ClientStatus == EConcertClientStatus::Disconnected && Index != INDEX_NONE)
		{
			Clients.RemoveAt(Index);
		}
		Settings->SaveConfig();
	}


	void AddRemoteClient(const FConcertSessionClientInfo& ClientInfo)
	{
		UConcertCVarConfig* Config = GetMutableDefault<UConcertCVarConfig>();
		FConcertCVarDetails Setting;
		Setting.Details = ClientInfo;

		Config->RemoteDetails.Emplace(MoveTemp(Setting));
	}

	void UpdateSessionClientList()
	{
		TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();

		check(Session.IsValid());

		TArray<FConcertSessionClientInfo> ConnectedClients = Session->GetSessionClients();

		UConcertCVarConfig* Config = GetMutableDefault<UConcertCVarConfig>();
		Config->RemoteDetails.Reset();
		Config->RemoteDetails.Reserve(ConnectedClients.Num());

		for (FConcertSessionClientInfo& Client : ConnectedClients)
		{
			AddRemoteClient(Client);
		}

		if (Customization)
		{
			Customization->PopulateClientList();
		}
		Config->SaveConfig();
	}

	void DisconnectFromSession()
	{
		check(WeakSession.IsValid());
		{
			TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
			UE_LOGF(LogConsoleVariablesEditor, VeryVerbose, "Multi-user Console Variable Editor disconnecting from Session: %ls", *Session->GetSessionInfo().SessionName);
			Session->OnSessionClientChanged().Remove(ClientChangeDelegate);
		}
	}

	void OnSessionConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus ConnectionStatus)
	{
		if (ConnectionStatus == EConcertConnectionStatus::Connected)
		{
			DisconnectFromSession();
			ConnectToSession(InSession);
		}
		else if (ConnectionStatus == EConcertConnectionStatus::Disconnecting)
		{
			DisconnectFromSession();
		}

		ConnectionChanged.Broadcast(ConnectionStatus);
	}

	void HandleCVarChange(const FConcertSessionContext& Context, const FConcertCVarChangeEvent& InEvent)
	{
		TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
		if (Session.IsValid())
		{
			UConcertCVarConfig* Config = GetMutableDefault<UConcertCVarConfig>();
			if (InEvent.EndpointId == Session->GetSessionClientEndpointId())
			{
				Config->LocalSettings = InEvent.Settings;
				if (Customization.IsValid())
				{
					FConcertCVarDetails Item;
					Item.Details.ClientEndpointId = InEvent.EndpointId;
					Item.Settings = InEvent.Settings;
					Item.bCVarSyncEnabled = GetDefault<UConcertCVarSynchronization>()->bSyncCVarTransactions;
					Customization->UpdateClientSettings(EConcertClientStatus::Updated, Item);
				}

			}
			else
			{
				for (FConcertCVarDetails& Item : Config->RemoteDetails)
				{
					if (Item.Details.ClientEndpointId == InEvent.EndpointId)
					{
						Item.Settings = InEvent.Settings;
						if (Customization.IsValid())
						{
							Customization->UpdateClientSettings(EConcertClientStatus::Updated, Item);
						}
					}
				}
			}
		}
	}

	void HandleCVarSync(const FConcertSessionContext& Context, const FConcertCVarSyncChangeEvent& InEvent)
	{
		TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
		if (Session.IsValid())
		{
			UConcertCVarConfig* Config = GetMutableDefault<UConcertCVarConfig>();
			for(FConcertCVarDetails& Item : Config->RemoteDetails)
			{
				if (Item.Details.ClientEndpointId == InEvent.EndpointId
					&& Item.bCVarSyncEnabled != InEvent.bSyncCVarChanges)
				{
					Item.bCVarSyncEnabled = InEvent.bSyncCVarChanges;
					if (Customization.IsValid())
					{
						Customization->UpdateClientSettings(EConcertClientStatus::Updated, Item);
					}
				}
			}
		}
	}

	static EConsoleVariableChangeType ConvertChangeType(ERemoteCVarChangeType InType)
	{
		switch (InType)
		{
		case ERemoteCVarChangeType::Add: return EConsoleVariableChangeType::Add;
		case ERemoteCVarChangeType::Remove: return EConsoleVariableChangeType::Remove;
		default:
			return EConsoleVariableChangeType::Modify;
		}
	}

	static ERemoteCVarChangeType ConvertChangeType(EConsoleVariableChangeType InType)
	{
		switch (InType)
		{
		case EConsoleVariableChangeType::Add: return ERemoteCVarChangeType::Add;
		case EConsoleVariableChangeType::Remove: return ERemoteCVarChangeType::Remove;
		default:
			return ERemoteCVarChangeType::Update;
		}
	}

	void HandleCVarSet(const FConcertSessionContext& Context, const FConcertSetConsoleVariableEvent& InEvent)
	{
		// Ignore this message if we generated it
		if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
		{
			if (Context.SourceEndpointId == Session->GetSessionClientEndpointId())
			{
				return;
			}
		}

		if (CanReceiveConsoleVariable())
		{
			EConsoleVariableFlags Flags = static_cast<EConsoleVariableFlags>(InEvent.Flags);
			RemoteCVarChanged.Broadcast(ConvertChangeType(InEvent.ChangeType), InEvent.Variable, InEvent.Value, Flags);
		}
	}

	/**
	 * Register a new multi-user session.
	 */
	void Register(TSharedRef<IConcertClientSession> InSession)
	{
		WeakSession = InSession;

		InSession->RegisterCustomEventHandler<FConcertCVarChangeEvent>(this, &FManagerImpl::HandleCVarChange);
		InSession->RegisterCustomEventHandler<FConcertCVarSyncChangeEvent>(this, &FManagerImpl::HandleCVarSync);
		InSession->RegisterCustomEventHandler<FConcertSetConsoleVariableEvent>(this, &FManagerImpl::HandleCVarSet);
	}

	/**
	 * Unregister a multi-user session (if one exists)
	 */
	void Unregister()
	{
		if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
		{
			Session->UnregisterCustomEventHandler<FConcertSetConsoleVariableEvent>(this);
			Session->UnregisterCustomEventHandler<FConcertCVarSyncChangeEvent>(this);
			Session->UnregisterCustomEventHandler<FConcertCVarChangeEvent>(this);
		}

		WeakSession.Reset();
	}

	/** 
	 * Unregister call that is invoked by Multi-user when it is shutdown. 
	 */
	void Unregister(TSharedRef<IConcertClientSession> InSession)
	{
		Unregister();
	}

	/**
	 * Returns true if this client can receive console variable changes.
	 */
	bool CanReceiveConsoleVariable() const
	{
		UConcertCVarSynchronization const* Sync = GetDefault<UConcertCVarSynchronization>();
		UConcertCVarConfig const* Config = GetDefault<UConcertCVarConfig>();
		return Sync->bSyncCVarTransactions && Config->LocalSettings.bReceiveCVarChanges;
	}

	void SettingChange(const FConcertCVarDetails& InChanged)
	{
		TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
		if (Session.IsValid())
		{
			UConcertCVarConfig* Config = GetMutableDefault<UConcertCVarConfig>();
			if (InChanged.Details.ClientEndpointId == Session->GetSessionClientEndpointId())
			{
				Config->LocalSettings = InChanged.Settings;
			}
			else
			{
				for(FConcertCVarDetails& Item : Config->RemoteDetails)
				{
					if (Item.Details.ClientEndpointId == InChanged.Details.ClientEndpointId)
					{
						Item.Settings = InChanged.Settings;
					}
				}
			}

			Config->SaveConfig();

			FConcertCVarChangeEvent OutEvent;
			OutEvent.Settings   = InChanged.Settings;
			OutEvent.EndpointId = InChanged.Details.ClientEndpointId;
			Session->SendCustomEvent(OutEvent, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
		}
	}

	/**
	 * Sends the given console variable name with the specified value to all connected endpoints.  It is up to the
	 * endpoint to implement the change locally based on configured synchronization state.
	 */
	void SendConsoleVariableChange(ERemoteCVarChangeType InChangeType, FString InName, FString InValue, EConsoleVariableFlags InFlags)
	{
		TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
		if (!bIsEnabled || !Session.IsValid())
		{
			return;
		}

		const bool bRenderThreadVersion = InFlags & ECVF_RenderThreadSafe;
		if (bRenderThreadVersion)
		{
			// The receiver is never dealing with the render thread version of the CVar thus should not be applied to remote callers.
			return;
		}

		uint32 OutFlags = static_cast<uint32>(InFlags);
		FConcertSetConsoleVariableEvent OutEvent{ConvertChangeType(InChangeType), MoveTemp(InName), MoveTemp(InValue), InFlags};
		Session->SendCustomEvent(OutEvent, Session->GetSessionClientEndpointIds(),
								 EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
	}

	/**
	 * Sends the given console variable name with the specified checked state to all connected endpoints.  It is up to the
	 * endpoint to implement the change locally based on configured synchronization state.
	 */
	void SendListItemCheckStateChange(FString InName, ECheckBoxState InCheckState)
	{
		TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
		if (!bIsEnabled || !Session.IsValid())
		{
			return;
		}
		FConcertSetListItemCheckStateEvent OutEvent{MoveTemp(InName), MoveTemp(InCheckState)};
		Session->SendCustomEvent(OutEvent, Session->GetSessionClientEndpointIds(),
								 EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
	}

	/** Reference to the customization object so that we can sync with the session.*/
	TSharedPtr<FConcertConsoleVariableSessionCustomization> Customization;

	/** Weak pointer to the client session with which to send events. May be null or stale. */
	TWeakPtr<IConcertClientSession> WeakSession;

	/** Delegate when property sync changes. */
	FDelegateHandle SyncDelegate;

	/** Delegate for any changes in client state. */
	FDelegateHandle	ClientChangeDelegate;

	/** Multicast delegate invoked when Multi-user console variable change event is received. */
	FOnRemoteCVarChange RemoteCVarChanged;

	/** Multicast delegate invoked when Multi-user console variable list item in CVE is manually checked or unchecked. */
	FOnRemoteListItemCheckStateChange RemoteListItemCheckStateChanged;

	/** Multi-user connection change delegate */
	FOnMultiUserConnectionChange ConnectionChanged;

	/** Indicates if the multi-user console synchronization is enabled. */
	bool bIsEnabled = true;

	/** True if the Concert module was found and customizations have been registered. */
	bool bIsInitialized = false;
};


FManager::FManager()
{
	Implementation = MakeUnique<FManagerImpl>();
}

FManager::~FManager() = default;

FOnRemoteCVarChange& FManager::OnRemoteCVarChange()
{
	return Implementation->RemoteCVarChanged;
}

FOnRemoteListItemCheckStateChange& FManager::OnRemoteListItemCheckStateChange()
{
	return Implementation->RemoteListItemCheckStateChanged;
}

FOnMultiUserConnectionChange& FManager::OnConnectionChange()
{
	return Implementation->ConnectionChanged;
}

void FManager::SendConsoleVariableChange(ERemoteCVarChangeType InChangeType, FString InName, FString InValue, EConsoleVariableFlags InFlags)
{
	Implementation->SendConsoleVariableChange(InChangeType, MoveTemp(InName), MoveTemp(InValue), InFlags);
}

void FManager::SendListItemCheckStateChange(FString InName, ECheckBoxState InCheckedState)
{
	Implementation->SendListItemCheckStateChange(MoveTemp(InName), MoveTemp(InCheckedState));
}

void FManager::SetEnableMultiUserSupport(bool bIsEnabled)
{
	Implementation->bIsEnabled = bIsEnabled;
}

bool FManager::IsInitialized() const
{
	return Implementation->bIsInitialized;
}

bool FManager::IsSyncEnabled() const
{
	return Implementation->bIsEnabled;
}

bool FManager::IsSessionValid() const
{
	return Implementation->WeakSession.IsValid();
}

bool FManager::IsLocalUserInMultiUserSession()
{
	return IsSyncEnabled() && IsSessionValid();
}
};

