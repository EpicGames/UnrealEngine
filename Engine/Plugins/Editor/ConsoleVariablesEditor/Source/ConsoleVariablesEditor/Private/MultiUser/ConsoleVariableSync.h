// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "HAL/IConsoleManager.h"
#include "Styling/SlateTypes.h"
#include "Templates/UniquePtr.h"


#include "ConcertMessages.h"

enum class ERemoteCVarChangeType : uint8
{
	Update = 0,
	Add,
	Remove
};

DECLARE_MULTICAST_DELEGATE_FourParams(FOnRemoteCVarChange, ERemoteCVarChangeType, FString, FString, EConsoleVariableFlags);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRemoteListItemCheckStateChange, FString, ECheckBoxState);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMultiUserConnectionChange, EConcertConnectionStatus);

namespace UE::ConsoleVariables::MultiUser::Private
{

struct FManagerImpl;

struct FManager
{
	FManager();
	~FManager();

	// Do not allow this to be copied or moved.
	FManager(const FManager&) = delete;
	FManager(FManager&&) = delete;
	FManager &operator=(const FManager&) = delete;
	FManager &operator=(FManager&&) = delete;

	/** Delegate that is invoked when a remote client has sent a new console variable value.*/
	FOnRemoteCVarChange& OnRemoteCVarChange();
	
	/** Delegate invoked when a remote client's console variable list item in CVE is manually checked or unchecked. */
	FOnRemoteListItemCheckStateChange& OnRemoteListItemCheckStateChange();

	/** Delegate that is invoked when the connection status changes for this client. */
	FOnMultiUserConnectionChange& OnConnectionChange();

	/** Sends the named console variable with value to all connected Multi-user clients */
	void SendConsoleVariableChange(ERemoteCVarChangeType InChangeType, FString InName, FString InValue, EConsoleVariableFlags InFlags);

	/** Sends the named console variable with value to all connected Multi-user clients */
	void SendListItemCheckStateChange(FString InName, ECheckBoxState InCheckedState);

	/** Enables / disables multi-user message handling. */
	void SetEnableMultiUserSupport(bool bIsEnabled);

	bool IsInitialized() const;

	bool IsSyncEnabled() const;

	bool IsSessionValid() const;

	bool IsLocalUserInMultiUserSession();
	
private:
	TUniquePtr<FManagerImpl> Implementation;
};

}
