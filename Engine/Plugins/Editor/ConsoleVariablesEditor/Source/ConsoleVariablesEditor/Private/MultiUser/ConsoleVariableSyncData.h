// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

#include "ConcertMessageData.h"

#include "Styling/SlateTypes.h"

#include "ConsoleVariableSyncData.generated.h"

UCLASS(config=Engine, DisplayName="Multi-user Console Variable Synchronization")
class UConcertCVarSynchronization : public UObject
{
public:
	GENERATED_BODY()

	UPROPERTY(config,EditAnywhere,BlueprintReadWrite,Category="Multi-User",DisplayName="Multi-User Console Variables Synchronization")
	bool bSyncCVarTransactions = true;
};

USTRUCT()
struct FConcertCVarSettings
{
	GENERATED_BODY();

	/** Indicates if the given node can receive console variable values from other clients. */
	UPROPERTY(config,EditAnywhere,Category="Multi-user Console Variable Settings")
	bool bReceiveCVarChanges = true;
};

USTRUCT()
struct FConcertCVarDetails
{
	GENERATED_BODY()

	UPROPERTY()
	FConcertSessionClientInfo Details;

	UPROPERTY()
	bool bCVarSyncEnabled = true;

	UPROPERTY()
	FConcertCVarSettings Settings;
};

UCLASS(config=Engine, DisplayName="Multi-user Console Variable Settings")
class UConcertCVarConfig : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(config,EditAnywhere,Category="Multi-user Console Variable Settings")
	FConcertCVarSettings LocalSettings;

	UPROPERTY(Transient,EditAnywhere,Category="Multi-user Console Variable Settings")
	TArray<FConcertCVarDetails> RemoteDetails;
};

USTRUCT()
struct FConcertCVarChangeEvent
{
	GENERATED_BODY()

	/** The endpoint id sending this event. */
	UPROPERTY()
	FGuid EndpointId;

	/** The Console Variable multi-user settings for the given EndpointId. */
	UPROPERTY()
	FConcertCVarSettings Settings;
};

USTRUCT()
struct FConcertCVarSyncChangeEvent
{
	GENERATED_BODY()

	/** The endpoint id sending this event. */
	UPROPERTY()
	FGuid EndpointId;

	/** The value of the console variable mult-user sync for the above EndpointId. */
	UPROPERTY()
	bool bSyncCVarChanges = true;
};

UENUM()
enum class EConsoleVariableChangeType : uint8
{
	/** The console variable was modified */
	Modify = 0,

	/** A console variable was added to the CVE asset */
	Add,

	/** A console variable was removed to the CVE asset */
	Remove
};

USTRUCT()
struct FConcertSetConsoleVariableEvent
{
	GENERATED_BODY()

	/** */
	UPROPERTY()
	EConsoleVariableChangeType ChangeType = EConsoleVariableChangeType::Modify;

	/** Console variable name to apply value change. */
	UPROPERTY()
	FString Variable;

	/** The value to set the named console variable. */
	UPROPERTY()
    FString Value;

	/** The flags for this console variable. */
    UPROPERTY()
	uint32 Flags = 0;
};

USTRUCT()
struct FConcertSetListItemCheckStateEvent
{
	GENERATED_BODY()
	/** Console variable name to apply the checked state change. */
	UPROPERTY()
	FString Variable;
	/** The checked state of the list item. */
	UPROPERTY()
	ECheckBoxState CheckState = ECheckBoxState::Checked;
};
