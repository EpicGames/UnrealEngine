// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/ConsoleVariablesEditorList.h"

#include "Views/List/SConsoleVariablesEditorList.h"

FConsoleVariablesEditorList::~FConsoleVariablesEditorList()
{
	ListWidget.Reset();
}

TSharedRef<SWidget> FConsoleVariablesEditorList::GetOrCreateWidget()
{
	if (!ListWidget.IsValid())
	{
		SAssignNew(ListWidget, SConsoleVariablesEditorList, SharedThis(this));
	}

	return ListWidget.ToSharedRef();
}

void FConsoleVariablesEditorList::SetSearchString(const FString& SearchString)
{
	ListWidget->SetSearchStringInSearchInputField(SearchString);
}

void FConsoleVariablesEditorList::RebuildList(const FString& InConsoleCommandToScrollTo, bool bShouldCacheValues) const
{
	if (ListWidget.IsValid())
	{
		ListWidget->RebuildListWithListMode(GetListMode(), InConsoleCommandToScrollTo, bShouldCacheValues);
	}
}

void FConsoleVariablesEditorList::RefreshList() const
{
	if (ListWidget.IsValid())
	{
		ListWidget->RefreshList();
	}
}

void FConsoleVariablesEditorList::UpdatePresetValuesForSave(TObjectPtr<UConsoleVariablesAsset> InAsset) const
{
	if (ListWidget.IsValid())
	{
		ListWidget->UpdatePresetValuesForSave(InAsset);
	}
}
void FConsoleVariablesEditorList::UpdateCachedValue(const FString& InCommand, const FString& InValue)
{
	if (ListWidget.IsValid())
	{
		ListWidget->UpdateCachedValue(InCommand, InValue);
	}
}

