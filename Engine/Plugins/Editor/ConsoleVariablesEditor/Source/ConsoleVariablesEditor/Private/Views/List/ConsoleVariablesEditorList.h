// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"

struct FConsoleVariablesEditorListRow;
typedef TSharedPtr<FConsoleVariablesEditorListRow> FConsoleVariablesEditorListRowPtr;

class SConsoleVariablesEditorList;
class UConsoleVariablesAsset;

class FConsoleVariablesEditorList : public TSharedFromThis<FConsoleVariablesEditorList>
{
public:

	enum class EConsoleVariablesEditorListMode : uint8
	{
		// We're displaying the cvars listed in the loaded (or default) preset
		Preset = 1,
		// We're displaying the cvars that match the criteria of the global search
		GlobalSearch = 2
	};

	FConsoleVariablesEditorList() {}

	~FConsoleVariablesEditorList();

	TSharedRef<SWidget> GetOrCreateWidget();

	void SetSearchString(const FString& SearchString);

	EConsoleVariablesEditorListMode GetListMode() const
	{
		return CurrentListMode;
	}

	void SetListMode(EConsoleVariablesEditorListMode NewListMode)
	{
		CurrentListMode = NewListMode;;
	}

	/*
	 * Regenerate the list items and refresh the list. Call when adding or removing variables.
	 * @param bShouldCacheValues If true, the current list's current values will be cached and then restored when the list is rebuilt. Otherwise preset values will be used.
	 */
	void RebuildList(const FString& InConsoleCommandToScrollTo = "", bool bShouldCacheValues = true) const;

	/**
	 * Refresh filters and sorting.
	 * Useful for when the list state has gone stale but the variable count has not changed.
	 */
	void RefreshList() const;

	/** Updates the saved values in a UConsoleVariablesAsset so that the command/value map can be saved to disk */
	void UpdatePresetValuesForSave(TObjectPtr<UConsoleVariablesAsset> InAsset) const;

	/** Update cached value */
	void UpdateCachedValue(const FString& Command, const FString& Value);

private:

	EConsoleVariablesEditorListMode CurrentListMode = EConsoleVariablesEditorListMode::Preset;
	TSharedPtr<SConsoleVariablesEditorList> ListWidget;
};
