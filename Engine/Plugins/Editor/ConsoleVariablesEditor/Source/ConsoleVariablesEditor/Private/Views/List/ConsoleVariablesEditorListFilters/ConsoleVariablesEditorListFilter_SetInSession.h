// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConsoleVariablesEditorListFilter.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

class ConsoleVariablesEditorListFilter_SetInSession : public IConsoleVariablesEditorListFilter
{
public:

	ConsoleVariablesEditorListFilter_SetInSession()
	{
		// This filter should be off by default
		SetFilterActive(false);
		SetFilterMatchType(EConsoleVariablesEditorListFilterMatchType::MatchAll);
	}

	virtual FString GetFilterName() override
	{
		return "Show Only Set In Session";
	}

	virtual FText GetFilterButtonLabel() override
	{
		return LOCTEXT("ShowOnlySetInSessionFilter", "Show Only Set In Session");
	}

	virtual FText GetFilterButtonToolTip() override
	{
		return LOCTEXT("ShowOnlySetInSessionFilterTooltip", "Show only rows that have been set in the current session.");
	}

	virtual bool DoesItemPassFilter(const FConsoleVariablesEditorListRowPtr& InItem) override
	{
		if (InItem.IsValid())
		{
			if (const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedCommand = InItem->GetCommandInfo().Pin())
			{
				return PinnedCommand->bSetInCurrentSession;
			}
		}

		return false;
	}
};

#undef LOCTEXT_NAMESPACE
