// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConsoleVariablesEditorListFilter.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

class ConsoleVariablesEditorListFilter_ShowOnlyCommands : public IConsoleVariablesEditorListFilter
{
public:

	ConsoleVariablesEditorListFilter_ShowOnlyCommands()
	{
		// This filter should be off by default
		SetFilterActive(false);
		SetFilterMatchType(EConsoleVariablesEditorListFilterMatchType::MatchAll);
	}

	virtual FString GetFilterName() override
	{
		return "Show Only Commands";
	}

	virtual FText GetFilterButtonLabel() override
	{
		return LOCTEXT("ShowOnlyCommandsFilter", "Show Only Commands");
	}

	virtual FText GetFilterButtonToolTip() override
	{
		return LOCTEXT("ShowOnlyCommandsFilterTooltip", "Show only rows that represent console commands, like 'stat unit'.");
	}

	virtual bool DoesItemPassFilter(const FConsoleVariablesEditorListRowPtr& InItem) override
	{
		if (InItem.IsValid())
		{
			if (const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedCommand = InItem->GetCommandInfo().Pin())
			{
				return PinnedCommand->ObjectType != FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable;
			}
		}

		return false;
	}
};

#undef LOCTEXT_NAMESPACE
