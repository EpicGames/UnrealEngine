// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConsoleVariablesEditorListFilter.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

class ConsoleVariablesEditorListFilter_ShowOnlyModifiedVariables : public IConsoleVariablesEditorListFilter
{
public:

	ConsoleVariablesEditorListFilter_ShowOnlyModifiedVariables()
	{
		// This filter should be off by default
		SetFilterActive(false);
		SetFilterMatchType(EConsoleVariablesEditorListFilterMatchType::MatchAll);
	}

	virtual FString GetFilterName() override
	{
		return "Show Only Modified Variables";
	}

	virtual FText GetFilterButtonLabel() override
	{
		return LOCTEXT("ShowOnlyModifiedVariablesFilter", "Show Only Modified");
	}

	virtual FText GetFilterButtonToolTip() override
	{
		return LOCTEXT("ShowOnlyModifiedVariablesFilterTooltip", "Show only rows that have a current value that differs from the preset value.");
	}

	virtual bool DoesItemPassFilter(const FConsoleVariablesEditorListRowPtr& InItem) override
	{
		if (InItem.IsValid())
		{
			return InItem->DoesCurrentValueDifferFromPresetValue();
		}

		return false;
	}
};

#undef LOCTEXT_NAMESPACE
