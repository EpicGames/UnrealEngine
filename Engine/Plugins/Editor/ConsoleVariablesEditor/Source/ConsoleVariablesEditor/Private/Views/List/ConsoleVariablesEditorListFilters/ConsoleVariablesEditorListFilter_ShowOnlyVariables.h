// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConsoleVariablesEditorListFilter.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

class ConsoleVariablesEditorListFilter_ShowOnlyVariables : public IConsoleVariablesEditorListFilter
{
public:

	ConsoleVariablesEditorListFilter_ShowOnlyVariables()
	{
		// This filter should be off by default
		SetFilterActive(false);
		SetFilterMatchType(EConsoleVariablesEditorListFilterMatchType::MatchAll);
	}

	virtual FString GetFilterName() override
	{
		return "Show Only Variables";
	}

	virtual FText GetFilterButtonLabel() override
	{
		return LOCTEXT("ShowOnlyVariablesFilter", "Show Only Variables");
	}

	virtual FText GetFilterButtonToolTip() override
	{
		return LOCTEXT("ShowOnlyVariablesFilterTooltip", "Show only rows that represent console variables.");
	}

	virtual bool DoesItemPassFilter(const FConsoleVariablesEditorListRowPtr& InItem) override
	{
		if (InItem.IsValid())
		{
			if (const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedCommand = InItem->GetCommandInfo().Pin())
			{
				return PinnedCommand->ObjectType == FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable;
			}
		}

		return false;
	}
};

#undef LOCTEXT_NAMESPACE
