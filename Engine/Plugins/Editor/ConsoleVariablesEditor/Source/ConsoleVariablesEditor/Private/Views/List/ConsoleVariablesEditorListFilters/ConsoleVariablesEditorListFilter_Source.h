// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConsoleVariablesEditorListFilter.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

class FConsoleVariablesEditorListFilter_Source : public IConsoleVariablesEditorListFilter
{
public:

	FConsoleVariablesEditorListFilter_Source(const EConsoleVariableFlags InSourceFlag)
	: SourceFlag(InSourceFlag){}

	virtual FString GetFilterName() override
	{
		return FConsoleVariablesEditorCommandInfo::ConvertConsoleVariableSetByFlagToText(SourceFlag).ToString();
	}

	virtual FText GetFilterButtonLabel() override
	{
		return FText::Format(LOCTEXT("ShowSourceTextFilterFormat", "Show {0}"), FText::FromString(GetFilterName()));
	}

	virtual FText GetFilterButtonToolTip() override
	{
		return FText::Format(
			LOCTEXT("ShowSourceTextFilterTooltipFormat", "Show rows that have a Source field matching '{0}'"),
			FText::FromString(GetFilterName()));
	}

	virtual bool DoesItemPassFilter(const FConsoleVariablesEditorListRowPtr& InItem) override
	{
		if (InItem.IsValid())
		{
			if (const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedCommand = InItem->GetCommandInfo().Pin())
			{
				const EConsoleVariableFlags Source = PinnedCommand->GetSource();
				return Source == SourceFlag;
			}
		}

		return false;
	}
	
private:

	EConsoleVariableFlags SourceFlag;
};

#undef LOCTEXT_NAMESPACE
