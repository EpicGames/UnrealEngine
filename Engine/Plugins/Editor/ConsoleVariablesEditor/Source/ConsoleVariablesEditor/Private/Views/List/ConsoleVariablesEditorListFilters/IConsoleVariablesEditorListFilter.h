// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../ConsoleVariablesEditorListRow.h"

class IConsoleVariablesEditorListFilter
{
public:

	enum class EConsoleVariablesEditorListFilterMatchType
	{
		// Filters of this type are placed into a group in which if any of the group's filters pass, the row is considered passing
		MatchAny,
		// Filters of this type are placed into a group in which all of the group's filters must pass for the row to be considered passing
		MatchAll
	};
	
	virtual ~IConsoleVariablesEditorListFilter() = default;

	virtual FString GetFilterName() = 0;

	/** Returns localized text to display for when this filter' active state is defined by a toggle button. */
	virtual FText GetFilterButtonLabel() = 0;
	/** Returns localized text to display when the user mouses over a toggle button that defines whether this filter is active. */
	virtual FText GetFilterButtonToolTip() = 0;
	
	void SetFilterActive(const bool bNewEnabled)
	{
		bIsFilterActive = bNewEnabled;
	}

	void ToggleFilterActive()
	{
		bIsFilterActive = !bIsFilterActive;
	}

	bool GetIsFilterActive() const
	{
		return bIsFilterActive;
	}

	void SetFilterMatchType(const EConsoleVariablesEditorListFilterMatchType InType)
	{
		FilterMatchType = InType;
	}

	EConsoleVariablesEditorListFilterMatchType GetFilterMatchType() const
	{
		return FilterMatchType;
	}

	virtual bool DoesItemPassFilter(const FConsoleVariablesEditorListRowPtr& InItem)
	{
		return false;
	}

private:

	bool bIsFilterActive = true;
	EConsoleVariablesEditorListFilterMatchType FilterMatchType = EConsoleVariablesEditorListFilterMatchType::MatchAny;
			
};
