// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleVariablesEditorListRow.h"

#include "ConsoleVariablesAsset.h"
#include "ConsoleVariablesEditorModule.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "MultiUser/ConsoleVariableSync.h"
#include "Views/List/ConsoleVariablesEditorList.h"
#include "Views/List/SConsoleVariablesEditorList.h"

FConsoleVariablesEditorListRow::~FConsoleVariablesEditorListRow()
{
	FlushReferences();
}

void FConsoleVariablesEditorListRow::FlushReferences()
{
	if (ChildRows.Num())
	{
		ChildRows.Empty();
	}
}

TWeakPtr<FConsoleVariablesEditorCommandInfo> FConsoleVariablesEditorListRow::GetCommandInfo() const
{
	return CommandInfo;
}

FConsoleVariablesEditorListRow::EConsoleVariablesEditorListRowType FConsoleVariablesEditorListRow::GetRowType() const
{
	return RowType;
}

int32 FConsoleVariablesEditorListRow::GetChildDepth() const
{
	return ChildDepth;
}

void FConsoleVariablesEditorListRow::SetChildDepth(const int32 InDepth)
{
	ChildDepth = InDepth;
}

int32 FConsoleVariablesEditorListRow::GetSortOrder() const
{
	return SortOrder;
}

void FConsoleVariablesEditorListRow::SetSortOrder(const int32 InNewOrder)
{
	SortOrder = InNewOrder;
}

TWeakPtr<FConsoleVariablesEditorListRow> FConsoleVariablesEditorListRow::GetDirectParentRow() const
{
	return DirectParentRow;
}

void FConsoleVariablesEditorListRow::SetDirectParentRow(
	const TWeakPtr<FConsoleVariablesEditorListRow>& InDirectParentRow)
{
	DirectParentRow = InDirectParentRow;
}

const TArray<FConsoleVariablesEditorListRowPtr>& FConsoleVariablesEditorListRow::GetChildRows() const
{
	return ChildRows;
}

int32 FConsoleVariablesEditorListRow::GetChildCount() const
{
	return ChildRows.Num();
}

void FConsoleVariablesEditorListRow::SetChildRows(const TArray<FConsoleVariablesEditorListRowPtr>& InChildRows)
{
	ChildRows = InChildRows;
}

void FConsoleVariablesEditorListRow::AddToChildRows(const FConsoleVariablesEditorListRowPtr& InRow)
{
	ChildRows.Add(InRow);
}

void FConsoleVariablesEditorListRow::InsertChildRowAtIndex(const FConsoleVariablesEditorListRowPtr& InRow,
                                                           const int32 AtIndex)
{
	ChildRows.Insert(InRow, AtIndex);
}

bool FConsoleVariablesEditorListRow::GetIsTreeViewItemExpanded() const
{
	return bIsTreeViewItemExpanded;
}

void FConsoleVariablesEditorListRow::SetIsTreeViewItemExpanded(const bool bNewExpanded)
{
	bIsTreeViewItemExpanded = bNewExpanded;
}

bool FConsoleVariablesEditorListRow::GetShouldFlashOnScrollIntoView() const
{
	return bShouldFlashOnScrollIntoView;
}

void FConsoleVariablesEditorListRow::SetShouldFlashOnScrollIntoView(const bool bNewShouldFlashOnScrollIntoView)
{
	bShouldFlashOnScrollIntoView = bNewShouldFlashOnScrollIntoView;
}

bool FConsoleVariablesEditorListRow::GetShouldExpandAllChildren() const
{
	return bShouldExpandAllChildren;
}

void FConsoleVariablesEditorListRow::SetShouldExpandAllChildren(const bool bNewShouldExpandAllChildren)
{
	bShouldExpandAllChildren = bNewShouldExpandAllChildren;
}

void FConsoleVariablesEditorListRow::ResetToStartupValueAndSource() const
{
	if (const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedCommand = GetCommandInfo().Pin())
	{
		PinnedCommand->SetSourceFlag(PinnedCommand->StartupSource);
		PinnedCommand->ExecuteCommand(PinnedCommand->StartupValueAsString, true, false);
	}
}

const FString& FConsoleVariablesEditorListRow::GetPresetValue() const
{
	return PresetValue;
}

void FConsoleVariablesEditorListRow::SetPresetValue(const FString& InPresetValue)
{
	PresetValue = InPresetValue;
}

bool FConsoleVariablesEditorListRow::MatchSearchTokensToSearchTerms(
	const TArray<FString> InTokens, ESearchCase::Type InSearchCase)
{
	// If the search is cleared we'll consider the row to pass search
	bool bMatchFound = InTokens.Num() == 0;

	if (!bMatchFound)
	{
		const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedInfo = CommandInfo.Pin();

		FString SearchTerms = PinnedInfo->Command + PinnedInfo->GetSourceAsText().ToString();

		if (const IConsoleVariable* AsVariable = PinnedInfo->GetConsoleVariablePtr())
		{
			// Add ValueAsString and HelpText to SearchTerms
			SearchTerms += AsVariable->GetString() + PinnedInfo->GetHelpText();
		}

		// Match any
		for (const FString& Token : InTokens)
		{
			// Match all of these
			const FString SpaceDelimiter = " ";
			TArray<FString> OutSpacedArray;
			if (Token.Contains(SpaceDelimiter) && Token.ParseIntoArray(OutSpacedArray, *SpaceDelimiter, true) > 1)
			{
				bMatchFound = Algo::AllOf(OutSpacedArray, [&SearchTerms, InSearchCase](const FString& Comparator)
				{
					return SearchTerms.Contains(Comparator, InSearchCase);
				});
			}
			else
			{
				bMatchFound = SearchTerms.Contains(Token, InSearchCase);
			}

			if (bMatchFound)
			{
				break;
			}
		}
	}

	bDoesRowMatchSearchTerms = bMatchFound;

	return bMatchFound;
}

void FConsoleVariablesEditorListRow::ExecuteSearchOnChildNodes(const FString& SearchString) const
{
	TArray<FString> Tokens;

	SearchString.ParseIntoArray(Tokens, TEXT(" "), true);

	ExecuteSearchOnChildNodes(Tokens);
}

void FConsoleVariablesEditorListRow::ExecuteSearchOnChildNodes(const TArray<FString>& Tokens) const
{
	for (const FConsoleVariablesEditorListRowPtr& ChildRow : GetChildRows())
	{
		if (!ensure(ChildRow.IsValid()))
		{
			continue;
		}

		if (ChildRow->GetRowType() == EConsoleVariablesEditorListRowType::CommandGroup)
		{
			if (ChildRow->MatchSearchTokensToSearchTerms(Tokens))
			{
				// If the group name matches then we pass an empty string to search child nodes since we want them all to be visible
				ChildRow->ExecuteSearchOnChildNodes("");
			}
			else
			{
				// Otherwise we iterate over all child nodes to determine which should and should not be visible
				ChildRow->ExecuteSearchOnChildNodes(Tokens);
			}
		}
		else
		{
			ChildRow->MatchSearchTokensToSearchTerms(Tokens);
		}
	}
}

bool FConsoleVariablesEditorListRow::GetDoesRowPassFilters() const
{
	return bDoesRowPassFilters;
}

void FConsoleVariablesEditorListRow::SetDoesRowPassFilters(const bool bPass)
{
	bDoesRowPassFilters = bPass;
}

bool FConsoleVariablesEditorListRow::DoesCurrentValueDifferFromPresetValue() const
{
	return bDoesCurrentValueDifferFromPresetValue;
}

void FConsoleVariablesEditorListRow::SetDoesCurrentValueDifferFromPresetValue(
	const bool bNewValue)
{
	bDoesCurrentValueDifferFromPresetValue = bNewValue;
}

bool FConsoleVariablesEditorListRow::GetIsSelected() const
{
	return bIsSelected;
}

void FConsoleVariablesEditorListRow::SetIsSelected(const bool bNewSelected)
{
	bIsSelected = bNewSelected;
}

ECheckBoxState FConsoleVariablesEditorListRow::GetWidgetCheckedState() const
{
	return WidgetCheckedState;
}

void FConsoleVariablesEditorListRow::SetWidgetCheckedState(const ECheckBoxState NewState,
                                                           const bool bShouldUpdateHierarchyCheckedStates)
{
	WidgetCheckedState = NewState;
}

bool FConsoleVariablesEditorListRow::IsRowChecked() const
{
	return GetWidgetCheckedState() == ECheckBoxState::Checked;
}

bool FConsoleVariablesEditorListRow::ShouldBeVisible() const
{
	return (bDoesRowMatchSearchTerms && bDoesRowPassFilters) || HasVisibleChildren();
}

EVisibility FConsoleVariablesEditorListRow::GetDesiredVisibility() const
{
	return ShouldBeVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

const FString& FConsoleVariablesEditorListRow::GetCachedValue() const
{
	return CachedValue;
}

void FConsoleVariablesEditorListRow::SetCachedValue(const FString& NewCachedValue)
{
	CachedValue = NewCachedValue;
}

TArray<FConsoleVariablesEditorListRowPtr> FConsoleVariablesEditorListRow::GetSelectedTreeViewItems() const
{
	return ListViewPtr.Pin()->GetSelectedTreeViewItems();
}

FReply FConsoleVariablesEditorListRow::OnActionButtonClicked()
{
	const bool bIsListValid =
		GetListViewPtr().IsValid() && GetListViewPtr().Pin()->GetListModelPtr().IsValid();

	if (!ensure(bIsListValid))
	{
		return FReply::Unhandled();
	}

	FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();
	const TObjectPtr<UConsoleVariablesAsset> EditableAsset = ConsoleVariablesEditorModule.GetPresetAsset();
	check(EditableAsset);

	const FConsoleVariablesEditorList::EConsoleVariablesEditorListMode ListMode =
		bIsListValid
			? GetListViewPtr().Pin()->GetListModelPtr().Pin()->GetListMode()
			: FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset;

	bool bWillRemove = true;

	// In global search mode, if this command isn't already in the preset, we want to add the preset instead
	const bool bIsGlobalSearch = ListMode == FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::GlobalSearch;
	if (bIsGlobalSearch)
	{
		FConsoleVariablesEditorAssetSaveData MatchingData;
		const FString& ClickedCommandName = GetCommandInfo().Pin()->Command;
		if (!ConsoleVariablesEditorModule.GetPresetAsset()->FindSavedDataByCommandString(ClickedCommandName, MatchingData, ESearchCase::IgnoreCase))
		{
			bWillRemove = false;
		}
	}

	for (const FConsoleVariablesEditorListRowPtr& RowPtr : GetRowsAffectedByActions())
	{
		if (!RowPtr.IsValid())
		{
			continue;
		}

		TSharedPtr<FConsoleVariablesEditorCommandInfo> Info = RowPtr->GetCommandInfo().Pin();
		const FString& CommandName = Info->Command;

		if (bWillRemove)
		{
			const FString& StartupValue = Info->StartupValueAsString;
			const EConsoleVariableFlags Source = Info->GetSource();

			RowPtr->ResetToStartupValueAndSource();
			EditableAsset->RemoveConsoleVariable(CommandName);

			ConsoleVariablesEditorModule.SendMultiUserConsoleVariableChange(ERemoteCVarChangeType::Remove, CommandName, StartupValue, Source);

			// Refresh preset list + numbers now that the row has been removed
			ListViewPtr.Pin()->RebuildListWithListMode(ListViewPtr.Pin()->GetListModelPtr().Pin()->GetListMode());
		}
		else
		{
			ensureMsgf(bIsGlobalSearch, TEXT("Add to preset operation should only occur in global search mode"));

			EditableAsset->AddOrSetConsoleObjectSavedData(
				{
					CommandName,
					"",
					ECheckBoxState::Checked
				}
			);
		}
	}

	return FReply::Handled();
}

bool FConsoleVariablesEditorListRow::IsGlobalSearch() const
{
	if (!ListViewPtr.IsValid())
	{
		return false;
	}

	const TWeakPtr<FConsoleVariablesEditorList> ListModelPtr = ListViewPtr.Pin()->GetListModelPtr();
	if (!ListModelPtr.IsValid())
	{
		return false;
	}

	return ListModelPtr.Pin()->GetListMode() == FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::GlobalSearch;
}

bool FConsoleVariablesEditorListRow::IsInPreset() const
{
	const FString& CommandName = GetCommandInfo().Pin()->Command;
	FConsoleVariablesEditorAssetSaveData MatchingData;

	const FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();
	return ConsoleVariablesEditorModule.GetPresetAsset()->FindSavedDataByCommandString(CommandName, MatchingData);
}

TArray<FConsoleVariablesEditorListRowPtr> FConsoleVariablesEditorListRow::GetRowsAffectedByActions()
{
	TArray<FConsoleVariablesEditorListRowPtr> AffectedRows = GetSelectedTreeViewItems();
	const bool bIsRowSelected = AffectedRows.ContainsByPredicate([this](const FConsoleVariablesEditorListRowPtr& RowPtr)
	{
		return RowPtr.Get() == this;
	});

	if (!bIsRowSelected)
	{
		AffectedRows.Empty(1);
		AffectedRows.Add(AsShared());
	}

	return AffectedRows;
}

void FConsoleVariablesEditorListRow::ResetToPresetValue()
{
	const FString& Value = GetPresetValue();
	GetCommandInfo().Pin()->ExecuteCommand(Value);
	SetCachedValue(Value);
}
