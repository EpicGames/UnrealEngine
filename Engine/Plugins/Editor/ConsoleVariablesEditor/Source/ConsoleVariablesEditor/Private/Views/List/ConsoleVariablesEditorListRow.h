// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConsoleVariablesEditorCommandInfo.h"
#include "Widgets/Input/SCheckBox.h"

class SConsoleVariablesEditorListValueInput;
class SConsoleVariablesEditorList;

struct FConsoleVariablesEditorListRow;
typedef TSharedPtr<FConsoleVariablesEditorListRow> FConsoleVariablesEditorListRowPtr;

struct FConsoleVariablesEditorListRow final : TSharedFromThis<FConsoleVariablesEditorListRow>
{
	enum EConsoleVariablesEditorListRowType
	{
		None,
		HeaderRow, 
		CommandGroup, // Group of commands or subgroups
		SingleCommand
	};

	~FConsoleVariablesEditorListRow();

	void FlushReferences();
	
	FConsoleVariablesEditorListRow(
		const TWeakPtr<FConsoleVariablesEditorCommandInfo> InCommandInfo, const FString& InPresetValue, const EConsoleVariablesEditorListRowType InRowType, 
		const ECheckBoxState StartingWidgetCheckboxState, const TSharedRef<SConsoleVariablesEditorList>& InListView, 
		const int32 IndexInList, const TWeakPtr<FConsoleVariablesEditorListRow>& InDirectParentRow)
	: CommandInfo(InCommandInfo)
	, PresetValue(InPresetValue)
	, RowType(InRowType)
	, WidgetCheckedState(StartingWidgetCheckboxState)
	, ListViewPtr(InListView)
	, SortOrder(IndexInList)
	, DirectParentRow(InDirectParentRow)
	{
		if (PresetValue.IsEmpty())
		{
			if (const auto Variable = CommandInfo.Pin()->GetConsoleVariablePtr())
			{
				PresetValue = Variable->GetString();
			}
		}

		SetCachedValue(PresetValue);
	}

	[[nodiscard]] TWeakPtr<FConsoleVariablesEditorCommandInfo> GetCommandInfo() const;

	[[nodiscard]] EConsoleVariablesEditorListRowType GetRowType() const;

	[[nodiscard]] int32 GetChildDepth() const;
	void SetChildDepth(const int32 InDepth);

	[[nodiscard]] int32 GetSortOrder() const;
	void SetSortOrder(const int32 InNewOrder);

	TWeakPtr<FConsoleVariablesEditorListRow> GetDirectParentRow() const;
	void SetDirectParentRow(const TWeakPtr<FConsoleVariablesEditorListRow>& InDirectParentRow);
	
	/* bHasGeneratedChildren must be true to get actual children. */
	[[nodiscard]] const TArray<FConsoleVariablesEditorListRowPtr>& GetChildRows() const;
	/* bHasGeneratedChildren must be true to get an accurate value. */
	[[nodiscard]] int32 GetChildCount() const;
	void SetChildRows(const TArray<FConsoleVariablesEditorListRowPtr>& InChildRows);
	void AddToChildRows(const FConsoleVariablesEditorListRowPtr& InRow);
	void InsertChildRowAtIndex(const FConsoleVariablesEditorListRowPtr& InRow, const int32 AtIndex = 0);

	[[nodiscard]] bool GetIsTreeViewItemExpanded() const;
	void SetIsTreeViewItemExpanded(const bool bNewExpanded);
	
	[[nodiscard]] bool GetShouldFlashOnScrollIntoView() const;
	void SetShouldFlashOnScrollIntoView(const bool bNewShouldFlashOnScrollIntoView);

	/** Broadcast to the row widget to play its flash animation. */
	FSimpleMulticastDelegate OnFlashRequested;
	void RequestFlash() { OnFlashRequested.Broadcast(); }

	[[nodiscard]] bool GetShouldExpandAllChildren() const;
	void SetShouldExpandAllChildren(const bool bNewShouldExpandAllChildren);

	void ResetToStartupValueAndSource() const;
	
	[[nodiscard]] const FString& GetPresetValue() const;
	void SetPresetValue(const FString& InPresetValue);

	/*
	 *Individual members of InTokens will be considered "AnyOf" or "OR" searches. If SearchTerms contains any individual member it will match.
	 *Members will be tested for a space character (" "). If a space is found, a subsearch will be run.
	 *This subsearch will be an "AllOf" or "AND" type search in which all strings, separated by a space, must be found in the search terms.
	 */
	bool MatchSearchTokensToSearchTerms(const TArray<FString> InTokens, ESearchCase::Type InSearchCase = ESearchCase::IgnoreCase);

	/* This overload creates tokens from a string first, then calls ExecuteSearchOnChildNodes(const TArray<FString>& Tokens). */
	void ExecuteSearchOnChildNodes(const FString& SearchString) const;
	void ExecuteSearchOnChildNodes(const TArray<FString>& Tokens) const;

	[[nodiscard]] bool GetDoesRowPassFilters() const;
	void SetDoesRowPassFilters(const bool bPass);
	
	[[nodiscard]] bool DoesCurrentValueDifferFromPresetValue() const;
	void SetDoesCurrentValueDifferFromPresetValue(const bool bNewValue);

	[[nodiscard]] bool GetIsSelected() const;
	void SetIsSelected(const bool bNewSelected);

	[[nodiscard]] ECheckBoxState GetWidgetCheckedState() const;
	void SetWidgetCheckedState(const ECheckBoxState NewState, const bool bShouldUpdateHierarchyCheckedStates = false);

	[[nodiscard]] bool IsRowChecked() const;

	[[nodiscard]] bool ShouldBeVisible() const;
	[[nodiscard]] EVisibility GetDesiredVisibility() const;

	[[nodiscard]] bool HasVisibleChildren() const
	{
		return false;
	}

	[[nodiscard]] TWeakPtr<SConsoleVariablesEditorList> GetListViewPtr() const
	{
		return ListViewPtr;
	}
	
	[[nodiscard]] const FString& GetCachedValue() const;
	void SetCachedValue(const FString& CachedValue);

	[[nodiscard]] TArray<FConsoleVariablesEditorListRowPtr> GetSelectedTreeViewItems() const;

	/** Return true if the row is being displayed in Global Search mode. */
	[[nodiscard]] bool IsGlobalSearch() const;

	/** Return true if the command represented by the row is included in the current preset. */
	[[nodiscard]] bool IsInPreset() const;

	/**
	 * Get an array of all rows that should be affected if an action is performed on this row.
	 * This takes multi-selection into account and imitates the Outliner visibility widget's behavior by only
	 * returning the row itself if the row is outside of the current selection.
	 */
	[[nodiscard]] TArray<FConsoleVariablesEditorListRowPtr> GetRowsAffectedByActions();

	FReply OnActionButtonClicked();
	
	void ResetToPresetValue();

private:

	TWeakPtr<FConsoleVariablesEditorCommandInfo> CommandInfo;
	FString PresetValue = "";
	EConsoleVariablesEditorListRowType RowType = SingleCommand;
	TArray<FConsoleVariablesEditorListRowPtr> ChildRows;

	ECheckBoxState WidgetCheckedState = ECheckBoxState::Checked;
	
	TWeakPtr<SConsoleVariablesEditorList> ListViewPtr;
	FString CachedValue = "";
	
	bool bIsTreeViewItemExpanded = false;
	bool bShouldFlashOnScrollIntoView = false;

	int32 ChildDepth = 0;

	int32 SortOrder = -1;

	bool bDoesRowMatchSearchTerms = true;
	bool bDoesRowPassFilters = true;

	bool bDoesCurrentValueDifferFromPresetValue = false;
	
	bool bIsSelected = false;
	TWeakPtr<FConsoleVariablesEditorListRow> DirectParentRow;

	// Used to expand all children on shift+click.
	bool bShouldExpandAllChildren = false;
};
