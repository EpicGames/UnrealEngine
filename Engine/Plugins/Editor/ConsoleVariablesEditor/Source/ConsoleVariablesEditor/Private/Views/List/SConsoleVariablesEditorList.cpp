// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConsoleVariablesEditorList.h"

// Filters
#include "ConsoleVariablesEditorListFilters/ConsoleVariablesEditorListFilter_ShowOnlyModifiedVariables.h"
#include "ConsoleVariablesEditorListFilters/ConsoleVariablesEditorListFilter_SetInSession.h"
#include "ConsoleVariablesEditorListFilters/ConsoleVariablesEditorListFilter_ShowOnlyCommands.h"
#include "ConsoleVariablesEditorListFilters/ConsoleVariablesEditorListFilter_ShowOnlyVariables.h"
#include "ConsoleVariablesEditorListFilters/ConsoleVariablesEditorListFilter_Source.h"

#include "ConsoleVariablesEditorLog.h"
#include "ConsoleVariablesEditorModule.h"
#include "ConsoleVariablesEditorProjectSettings.h"
#include "ConsoleVariablesEditorStyle.h"
#include "SConsoleVariablesEditorListRow.h"
#include "../Widgets/SConsoleVariablesEditorGlobalSearchToggle.h"

#include "HAL/PlatformApplicationMisc.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

const FName SConsoleVariablesEditorList::CustomSortOrderColumnName(TEXT("Order"));
const FName SConsoleVariablesEditorList::CheckBoxColumnName(TEXT("Checkbox"));
const FName SConsoleVariablesEditorList::VariableNameColumnName(TEXT("Name"));
const FName SConsoleVariablesEditorList::ValueColumnName(TEXT("Value"));
const FName SConsoleVariablesEditorList::SourceColumnName(TEXT("Source"));
const FName SConsoleVariablesEditorList::ActionButtonColumnName(TEXT("Action"));

void SConsoleVariablesEditorList::Construct(const FArguments& InArgs, TSharedRef<FConsoleVariablesEditorList> ListModel)
{
	ListModelPtr = ListModel;
	
	// Set Default Sorting info
	ActiveSortingColumnName = CustomSortOrderColumnName;
	ActiveSortingType = EColumnSortMode::Ascending;
	
	HeaderRow = SNew(SHeaderRow)
				.CanSelectGeneratedColumn(true).Visibility(EVisibility::Visible);
	
	GenerateHeaderRow();

	SetupFilters();
	
	ChildSlot
	[
		SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		.Padding(FMargin(8.f, 0.f, 8.f, 0.f))
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(0.f, 1.f, 0.f, 1.f)
			[
				SAssignNew(ListSearchBoxPtr, SSearchBox)
				.HintText(LOCTEXT("ConsoleVariablesEditorList_SearchHintText", "Search..."))
				.ToolTipText(LOCTEXT("ConsoleVariablesEditorList_TooltipText", "Search tracked variables, values, sources or help text"))
				.OnTextChanged_Raw(this, &SConsoleVariablesEditorList::OnListViewSearchTextChanged)
			]

			// Global Search Button
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(FMargin(8.f, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ContentPadding(FMargin(4, 1.5f))
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([this]()
				{
					const FString SearchString = GetSearchStringFromSearchInputField().ToLower().TrimStartAndEnd();
					ListSearchBoxPtr->SetText(FText::GetEmpty());
					VisibleTreeViewObjects.Empty();
					return TryEnterGlobalSearch(SearchString);
				})
				.ToolTipText(LOCTEXT("OpenInGlobalSearchButtonTooltip", "Search All Console Variables"))
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.Padding(0, 1, 4, 0)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FConsoleVariablesEditorStyle::Get().GetBrush("ConsoleVariables.GlobalSearch.Small"))
					]

					+SHorizontalBox::Slot()
					.Padding(4.f, 1.f, 0.f, 0.f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("GlobalSearchButtonText","Search All"))
					]
				]
			]

			// Show Options
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(8.f, 1.f, 0.f, 1.f)
			[
				SAssignNew( ViewOptionsComboButton, SComboButton )
				.ContentPadding(4.f)
				.ToolTipText(LOCTEXT("ShowOptions_Tooltip", "Show options to affect the visibility of items in the Console Variables Editor list"))
				.ComboButtonStyle( FAppStyle::Get(), "SimpleComboButtonWithIcon" ) // Use the tool bar item style for this button
				.OnGetMenuContent( this, &SConsoleVariablesEditorList::BuildShowOptionsMenu)
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image( FAppStyle::Get().GetBrush("Icons.Settings") )
				]
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(8.f, 4.f, 8.f, 4.f))
		[
			SAssignNew(GlobalSearchesHBox, SHorizontalBox)
			.Visibility_Lambda([this]()
			{
				const bool bShouldBeVisible =
					GlobalSearchesContainer.IsValid() && GlobalSearchesContainer->GetChildren()->Num() > 0;

				return bShouldBeVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
			})

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				SAssignNew(GlobalSearchesContainer, SWrapBox)
				.InnerSlotPadding(FVector2d(6, 4))
				.UseAllottedSize(true)
			]

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(8.f, 1.f, 2.f, 1.f)
			[
				// Remove Button
				SAssignNew(RemoveGlobalSearchesButtonPtr, SCheckBox)
				.Padding(0)
				.ToolTipText(
					LOCTEXT("RemoveGlobalSearchesButtonTooltip", "Remove all global searches from the console variables editor."))
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
				.ForegroundColor(FSlateColor::UseForeground())
				.IsChecked(false)
				.OnCheckStateChanged_Lambda([this] (ECheckBoxState NewCheckState)
				{
					GlobalSearchesContainer->ClearChildren();
					CurrentGlobalSearches.Empty();

					RebuildListWithListMode(FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset);

					RemoveGlobalSearchesButtonPtr->SetIsChecked(false);
				})
				[
					SNew(SImage)
					.Visibility(EVisibility::SelfHitTestInvisible)
					.Image(FAppStyle::Get().GetBrush("Icons.X"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]

		+ SVerticalBox::Slot()
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex_Lambda([this]()
			{
				return DoesTreeViewHaveVisibleChildren() ? 0 : 1;
			})
			
			+ SWidgetSwitcher::Slot()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 2.0f, 0.0f, 2.0f)
			[
				SAssignNew(TreeViewPtr, STreeView<FConsoleVariablesEditorListRowPtr>)
				.HeaderRow(HeaderRow)
				.SelectionMode(ESelectionMode::Multi)
				.OnContextMenuOpening(this, &SConsoleVariablesEditorList::OnTreeViewContextMenuOpening)
				.OnSelectionChanged_Lambda([this] (const FConsoleVariablesEditorListRowPtr& Row, const ESelectInfo::Type SelectionType)
				{
					if(Row.IsValid())
					{
						Row->SetIsSelected(TreeViewPtr->GetSelectedItems().Contains(Row));
					}
				})
				.TreeItemsSource(&VisibleTreeViewObjects)
				.OnGenerateRow_Lambda([this](FConsoleVariablesEditorListRowPtr Row, const TSharedRef<STableViewBase>& OwnerTable)
					{
						check(Row.IsValid());
					
						return SNew(SConsoleVariablesEditorListRow, TreeViewPtr.ToSharedRef(), Row)
								.Visibility_Raw(Row.Get(), &FConsoleVariablesEditorListRow::GetDesiredVisibility);
					})
				.OnGetChildren_Raw(this, &SConsoleVariablesEditorList::OnGetRowChildren)
				.OnExpansionChanged_Raw(this, &SConsoleVariablesEditorList::OnRowChildExpansionChange, false)
				.OnSetExpansionRecursive(this, &SConsoleVariablesEditorList::OnRowChildExpansionChange, true)
			]

			// For when no rows exist in view
			+ SWidgetSwitcher::Slot()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 24.0f, 0.0f, 2.0f)
			[
				SAssignNew(EmptyAreaBorder, SBorder)
				.BorderImage(FStyleDefaults::GetNoBrush())
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
				SNew(SRichTextBlock)
				.DecoratorStyleSet(&FAppStyle::Get())
				.AutoWrapText(true)
				.Justification(ETextJustify::Center)
				.Text_Lambda([this]()
				{
					if (VisibleTreeViewObjects.Num() == 0)
					{
						if (TreeViewRootObjects.Num() == 0)
						{
							if (GetListModelPtr().Pin()->GetListMode() == FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::GlobalSearch)
							{
								// Global Search Empty List (without filter)
								return LOCTEXT("ConsoleVariablesEditorList_EmptyListGlobalSearch", "No matching console variables found in Unreal Engine.\n\nCheck your search criteria.");
							}

							// Preset Empty List (without Filter)
							return LOCTEXT("ConsoleVariablesEditorList_EmptyListPresetWithoutFilter", "Type in a variable to the console or <RichTextBlock.Bold>Search All</> to add to your list.");
						}
					}

					// Preset Empty List (with filter)
                    return LOCTEXT("ConsoleVariablesEditorList_EmptyListPresetWithFilter", "No matching console variables in your list.\n\nCheck your filter or <RichTextBlock.Bold>Search All</> console variables instead.");
				})
				]
			]
		]
	];

	if (EmptyAreaBorder.IsValid())
	{
		EmptyAreaBorder->SetOnMouseButtonUp(
			FPointerEventHandler::CreateSP(this, &SConsoleVariablesEditorList::OnEmptyAreaMouseButtonUp));
	}

	EvaluateIfRowsPassFilters();
}

SConsoleVariablesEditorList::~SConsoleVariablesEditorList()
{	
	HeaderRow.Reset();
	
	ListSearchBoxPtr.Reset();
	ViewOptionsComboButton.Reset();
	GlobalSearchesHBox.Reset();
	GlobalSearchesContainer.Reset();
	CurrentGlobalSearches.Empty();
	RemoveGlobalSearchesButtonPtr.Reset();
	ListBoxContainerPtr.Reset();

	FlushMemory(false);

	ShowFilters.Reset();
	TreeViewPtr.Reset();
	VisibleTreeViewObjects.Reset();
	LastPresetObjects.Reset();
	
	CachedCommandStates.Empty();
}

FReply SConsoleVariablesEditorList::TryEnterGlobalSearch(const FString& SearchString)
{
	FReply ReturnValue = FReply::Unhandled();

	// Can't enter global search if there are no active global searches or a search string from which to parse new searches
	if (SearchString.IsEmpty() && CurrentGlobalSearches.Num() == 0)
	{
		UE_LOGF(LogConsoleVariablesEditor, Verbose,
			"%s: Global search request is empty. Exiting Global Search.",
			__FUNCTION__);

		// Return to preset mode if in global search mode and rebuild list
		if (ListModelPtr.Pin()->GetListMode() == FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::GlobalSearch)
		{
			RebuildListWithListMode(FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset);
		}

		return ReturnValue;
	}

	// Strings that already have associated buttons	
	TArray<FString> ExistingButtonStrings;
	
	// All strings parsed from the search text
	TArray<FString> OutTokens;
	SearchString.ParseIntoArray(OutTokens, TEXT("|"), true);

	// Get tokens from current searches. This step allows us to properly populate the asset with all matching commands
	for (const TSharedRef<SConsoleVariablesEditorGlobalSearchToggle>& GlobalSearchButton : CurrentGlobalSearches)
	{
		const FString ButtonText = GlobalSearchButton->GetGlobalSearchText().ToString();
		ExistingButtonStrings.Add(ButtonText);

		// If the button text was explicitly typed into the search then set the existing button to be checked.
		if (OutTokens.ContainsByPredicate([this, &ButtonText](const FString& Token)
		{
			return Token.Equals(ButtonText, ESearchCase::IgnoreCase);
		}))
		{
			GlobalSearchButton->SetIsButtonChecked(true);
		}

		// If the button is checked, add its search text to the tokens array so it can be used to populate the matching command list
		if (GlobalSearchButton->GetIsToggleChecked())
		{
			OutTokens.AddUnique(ButtonText);
		}
	}
	
	FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();
	
	const bool bFoundMatches = ConsoleVariablesEditorModule.PopulateGlobalSearchAssetWithVariablesMatchingTokens(OutTokens);
	{
		// Convert tokens to global search toggle buttons
		for (const FString& TokenString : OutTokens)
		{
			// Only make new buttons when one doesn't exist for the current token
			if (!ExistingButtonStrings.Contains(TokenString))
			{
				TSharedRef<SConsoleVariablesEditorGlobalSearchToggle> NewGlobalSearchButton =
					SNew(SConsoleVariablesEditorGlobalSearchToggle, FText::FromString(TokenString))
				.OnToggleClickedOnce_Lambda([this]()
				{
					return TryEnterGlobalSearch();
				})
				.OnToggleCtrlClicked(this, &SConsoleVariablesEditorList::HandleRemoveGlobalSearchToggleButton)
				.OnToggleRightButtonClicked(this, &SConsoleVariablesEditorList::HandleRemoveGlobalSearchToggleButton);

				CurrentGlobalSearches.Add(NewGlobalSearchButton);
			}
		}

		// Put widgets in container
		RefreshGlobalSearchWidgets();

		ReturnValue = FReply::Handled();
	}
	
	if (!bFoundMatches)
	{
		UE_LOGF(LogConsoleVariablesEditor, Warning,
			"%s: Failed to find console variable objects with names containing search string %ls",
			__FUNCTION__, *SearchString);
	}

	RebuildListWithListMode(FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::GlobalSearch);

	return ReturnValue;
}

FReply SConsoleVariablesEditorList::HandleRemoveGlobalSearchToggleButton()
{
	CleanUpGlobalSearchesMarkedForDelete();
	RefreshGlobalSearchWidgets();
	return TryEnterGlobalSearch();
}

void SConsoleVariablesEditorList::CleanUpGlobalSearchesMarkedForDelete()
{
	for (int32 GlobalSearchItr = CurrentGlobalSearches.Num() - 1; GlobalSearchItr >= 0; GlobalSearchItr--)
	{
		if (CurrentGlobalSearches[GlobalSearchItr]->GetIsMarkedForDelete())
		{
			CurrentGlobalSearches.RemoveAt(GlobalSearchItr);
		}
	}
}

void SConsoleVariablesEditorList::RefreshGlobalSearchWidgets()
{
	GlobalSearchesContainer->ClearChildren();

	for (const TSharedRef<SConsoleVariablesEditorGlobalSearchToggle>& GlobalSearchButton : CurrentGlobalSearches)
	{	
		GlobalSearchesContainer->AddSlot()
		[
			GlobalSearchButton
		];
	}
}

void SConsoleVariablesEditorList::RebuildListWithListMode(
	FConsoleVariablesEditorList::EConsoleVariablesEditorListMode NewListMode, const FString& InConsoleCommandToScrollTo, bool bShouldCacheValues)
{
	if (bShouldCacheValues)
	{
		CacheCurrentListItemData();
	}

	// If we're in Preset mode and changing to something else, cache the existing tree objects to maintain state
	if (GetListModelPtr().Pin()->GetListMode() == FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset &&
		NewListMode != FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset)
	{
		LastPresetObjects = TreeViewRootObjects;
	}
	
	GetListModelPtr().Pin()->SetListMode(NewListMode);

	// Skip execution on load if we've cached the previous values
	GenerateTreeView(bShouldCacheValues);

	if (bShouldCacheValues)
	{
		RestorePreviousListItemData();
	}
	else
	{
		// If we didn't want to cache, clear the previous cache
		CachedCommandStates.Empty();
	}

	RefreshList();

	if (!InConsoleCommandToScrollTo.IsEmpty())
	{
		FConsoleVariablesEditorListRowPtr ScrollToItem = nullptr;

		for (const FConsoleVariablesEditorListRowPtr& Item : TreeViewRootObjects)
		{
			if (Item->GetCommandInfo().Pin()->Command.Equals(InConsoleCommandToScrollTo, ESearchCase::IgnoreCase))
			{
				ScrollToItem = Item;
				break;
			}
		}

		if (ScrollToItem.IsValid())
		{
			ScrollToItem->SetShouldFlashOnScrollIntoView(true);
			TreeViewPtr->RequestScrollIntoView(ScrollToItem);
		}
	}
}

void SConsoleVariablesEditorList::RefreshList()
{
	if (TreeViewRootObjects.Num() > 0)
	{
		// Apply last search
		ExecuteListViewSearchOnAllRows(GetSearchStringFromSearchInputField(), false);

		// Enforce Sort
		const FName& SortingName = GetActiveSortingColumnName();
		ExecuteSort(SortingName, GetSortModeForColumn(SortingName), false);

		// Show/Hide rows based on SetBy changes and filter settings
		EvaluateIfRowsPassFilters(false);

		// Refresh the header's check state
		OnListItemCheckBoxStateChange(ECheckBoxState::Undetermined);
	}

	FindVisibleObjectsAndRequestTreeRefresh();
}

TArray<FConsoleVariablesEditorListRowPtr> SConsoleVariablesEditorList::GetSelectedTreeViewItems() const
{
	return TreeViewPtr->GetSelectedItems();
}

TArray<FConsoleVariablesEditorListRowPtr> SConsoleVariablesEditorList::GetTreeViewItems() const
{
	return TreeViewRootObjects;
}

void SConsoleVariablesEditorList::SetTreeViewItems(const TArray<FConsoleVariablesEditorListRowPtr>& InItems)
{
	TreeViewRootObjects = InItems;

	TreeViewPtr->RequestListRefresh();
}

void SConsoleVariablesEditorList::UpdatePresetValuesForSave(const TObjectPtr<UConsoleVariablesAsset> InAsset) const
{
	TArray<FConsoleVariablesEditorAssetSaveData> NewSavedCommands;

	const TArray<FConsoleVariablesEditorListRowPtr>& Items =
		ListModelPtr.IsValid() && ListModelPtr.Pin()->GetListMode() ==
			FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset ? TreeViewRootObjects : LastPresetObjects;
	
	for (const FConsoleVariablesEditorListRowPtr& Item : Items)
	{
		if (const TWeakPtr<FConsoleVariablesEditorCommandInfo> CommandInfo = Item->GetCommandInfo(); CommandInfo.IsValid())
		{
			NewSavedCommands.Add(
				{
					CommandInfo.Pin()->Command,
					Item->GetCachedValue(),
					Item->GetWidgetCheckedState()
				}
			);
		}
	}

	InAsset->ReplaceSavedCommands(NewSavedCommands);
}

void SConsoleVariablesEditorList::UpdateCachedValue(const FString& InCommand, const FString& InValue)
{
	const TArray<FConsoleVariablesEditorListRowPtr>& Items =
		ListModelPtr.IsValid() && ListModelPtr.Pin()->GetListMode() ==
			FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset ? TreeViewRootObjects : LastPresetObjects;
	const FConsoleVariablesEditorListRowPtr* Item = Algo::FindByPredicate(Items, [InCommand](const FConsoleVariablesEditorListRowPtr& Item)
	{
		TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo = Item->GetCommandInfo().Pin();
		return CommandInfo ? CommandInfo->Command == InCommand : false;
	});
	if (Item && (*Item)->IsRowChecked())
	{
		(*Item)->SetCachedValue(InValue);
	}
}

FString SConsoleVariablesEditorList::GetSearchStringFromSearchInputField() const
{
	return ensureAlwaysMsgf(ListSearchBoxPtr.IsValid(),
		TEXT("%hs: ListSearchBoxPtr is not valid. Check to make sure it was created."), __FUNCTION__)
	? ListSearchBoxPtr->GetText().ToString() : "";
}

void SConsoleVariablesEditorList::SetSearchStringInSearchInputField(const FString InSearchString) const
{
	if (ensureAlwaysMsgf(ListSearchBoxPtr.IsValid(),
		TEXT("%hs: ListSearchBoxPtr is not valid. Check to make sure it was created."), __FUNCTION__))
	{
		ListSearchBoxPtr->SetText(FText::FromString(InSearchString));
	}
}

void SConsoleVariablesEditorList::ExecuteListViewSearchOnAllRows(
	const FString& SearchString, const bool bShouldRefreshAfterward)
{
	TArray<FString> Tokens;
	
	// unquoted search equivalent to a match-any-of search
	SearchString.ParseIntoArray(Tokens, TEXT("|"), true);
	
	for (const TSharedPtr<FConsoleVariablesEditorListRow>& ChildRow : TreeViewRootObjects)
	{
		if (!ensure(ChildRow.IsValid()))
		{
			continue;
		}
		
		const bool bGroupMatch = ChildRow->MatchSearchTokensToSearchTerms(Tokens);
		
		// If the group name matches then we pass in an empty string so all child nodes are visible.
		// If the name doesn't match, then we need to evaluate each child.
		ChildRow->ExecuteSearchOnChildNodes(bGroupMatch ? "" : SearchString);
	}

	if (bShouldRefreshAfterward)
	{
		FindVisibleObjectsAndRequestTreeRefresh();
	}
}

bool SConsoleVariablesEditorList::DoesTreeViewHaveVisibleChildren() const
{
	if (TreeViewPtr.IsValid())
	{
		for (const TSharedPtr<FConsoleVariablesEditorListRow>& Header : TreeViewRootObjects)
		{
			const EVisibility HeaderVisibility = Header->GetDesiredVisibility();
			
			if (HeaderVisibility != EVisibility::Hidden && HeaderVisibility != EVisibility::Collapsed)
			{
				return true;
			}
		}
	}
	
	return false;
}

void SConsoleVariablesEditorList::SetTreeViewItemExpanded(const TSharedPtr<FConsoleVariablesEditorListRow>& RowToExpand, const bool bNewExpansion) const
{
	if (TreeViewPtr.IsValid())
	{
		TreeViewPtr->SetItemExpansion(RowToExpand, bNewExpansion);
	}
}

void SConsoleVariablesEditorList::SetAllListViewItemsCheckState(const ECheckBoxState InNewState)
{
	for (const TSharedPtr<FConsoleVariablesEditorListRow>& Row : TreeViewRootObjects)
	{
		Row->SetWidgetCheckedState(InNewState);
	}
}

bool SConsoleVariablesEditorList::DoesListHaveCheckedMembers() const
{
	for (const TSharedPtr<FConsoleVariablesEditorListRow>& Row : TreeViewRootObjects)
	{
		if (Row->GetCommandInfo().Pin()->ObjectType ==
			FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable &&
			Row->GetWidgetCheckedState() == ECheckBoxState::Checked)
		{
			return true;
		}
	}

	return false;
}

bool SConsoleVariablesEditorList::DoesListHaveUncheckedMembers() const
{
	for (const TSharedPtr<FConsoleVariablesEditorListRow>& Row : TreeViewRootObjects)
	{
		if (Row->GetCommandInfo().Pin()->ObjectType ==
			FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable &&
			Row->GetWidgetCheckedState() == ECheckBoxState::Unchecked)
		{
			return true;
		}
	}

	return false;
}

void SConsoleVariablesEditorList::OnListItemCheckBoxStateChange(const ECheckBoxState InNewState)
{
	HeaderCheckBoxState = ECheckBoxState::Checked;

	if (DoesListHaveUncheckedMembers())
	{
		HeaderCheckBoxState = ECheckBoxState::Unchecked;

		if (DoesListHaveCheckedMembers())
		{
			HeaderCheckBoxState = ECheckBoxState::Undetermined;
		}
	}
}

void SConsoleVariablesEditorList::ToggleFilterActive(const FString& FilterName)
{
	if (const TSharedRef<IConsoleVariablesEditorListFilter>* Match =
		Algo::FindByPredicate(ShowFilters,
		[&FilterName](TSharedRef<IConsoleVariablesEditorListFilter> Comparator)
		{
			return Comparator->GetFilterName().Equals(FilterName);
		}))
	{
		const TSharedRef<IConsoleVariablesEditorListFilter> Filter = *Match;
		Filter->ToggleFilterActive();

		EvaluateIfRowsPassFilters();
	}
}

void SConsoleVariablesEditorList::EvaluateIfRowsPassFilters(const bool bShouldRefreshAfterward)
{
	// Separate filters by type
	
	TSet<TSharedRef<IConsoleVariablesEditorListFilter>> MatchAnyOfFilters;
	TSet<TSharedRef<IConsoleVariablesEditorListFilter>> MatchAllOfFilters;

	for (const TSharedRef<IConsoleVariablesEditorListFilter>& Filter : ShowFilters)
	{
		if (Filter->GetFilterMatchType() ==
			IConsoleVariablesEditorListFilter::EConsoleVariablesEditorListFilterMatchType::MatchAll)
		{
			MatchAllOfFilters.Add(Filter);
		}
		else
		{
			MatchAnyOfFilters.Add(Filter);
		}
	}
	
	for (const FConsoleVariablesEditorListRowPtr& Row : TreeViewRootObjects)
	{
		if (Row.IsValid() && Row->GetRowType() == FConsoleVariablesEditorListRow::SingleCommand)
		{
			// If any filters of this group are inactive, they automatically fail.
			// For "Show" filters, this means another active filter must pass in order for the row to pass 'bPassesAnyOf'.
			auto AnyOfProjection = [&Row](const TSharedRef<IConsoleVariablesEditorListFilter>& Filter)
			{
				return Filter->GetIsFilterActive() && Filter->DoesItemPassFilter(Row);
			};

			// If any filters of this group are inactive, they automatically pass.
			// For "Show Only" filters, only active filters will check the passing condition,
			// and all must pass in order for the row to pass 'bPassesAllOf'.
			auto AllOfProjection = [&Row](const TSharedRef<IConsoleVariablesEditorListFilter>& Filter)
			{
				return !Filter->GetIsFilterActive() || Filter->DoesItemPassFilter(Row);
			};
			
			const bool bPassesAnyOf = Algo::AnyOf(MatchAnyOfFilters, AnyOfProjection);
			const bool bPassesAllOf = Algo::AllOf(MatchAllOfFilters, AllOfProjection);
			
			Row->SetDoesRowPassFilters(bPassesAnyOf && bPassesAllOf);
		}
	}

	if (bShouldRefreshAfterward)
	{
		FindVisibleObjectsAndRequestTreeRefresh();
	}
}

EColumnSortMode::Type SConsoleVariablesEditorList::GetSortModeForColumn(FName InColumnName) const
{
	EColumnSortMode::Type ColumnSortMode = EColumnSortMode::None;

	if (GetActiveSortingColumnName().IsEqual(InColumnName))
	{
		ColumnSortMode = ActiveSortingType;
	}

	return ColumnSortMode;
}

void SConsoleVariablesEditorList::OnSortColumnCalled(EColumnSortPriority::Type Priority, const FName& ColumnName, EColumnSortMode::Type SortMode)
{
	ExecuteSort(ColumnName, CycleSortMode(ColumnName));
}

EColumnSortMode::Type SConsoleVariablesEditorList::CycleSortMode(const FName& InColumnName)
{
	// Custom handler for Custom Sort Order mode
	if (InColumnName.IsEqual(CustomSortOrderColumnName))
	{
		ActiveSortingType = EColumnSortMode::Ascending;
	}
	else
	{
		const EColumnSortMode::Type PreviousColumnSortMode = GetSortModeForColumn(InColumnName);
		ActiveSortingType = PreviousColumnSortMode ==
			EColumnSortMode::Ascending ? EColumnSortMode::Descending : EColumnSortMode::Ascending;
	}

	ActiveSortingColumnName = InColumnName;
	return ActiveSortingType;
}

void SConsoleVariablesEditorList::ExecuteSort(
	const FName& InColumnName, const EColumnSortMode::Type InColumnSortMode, const bool bShouldRefreshAfterward)
{	
	if (InColumnName.IsEqual(CustomSortOrderColumnName))
	{
		TreeViewRootObjects.StableSort(SortByOrderAscending);
	}
	if (InColumnName.IsEqual(SourceColumnName))
	{
		TreeViewRootObjects.StableSort(
			InColumnSortMode == EColumnSortMode::Ascending ? SortBySourceAscending : SortBySourceDescending);
	}
	if (InColumnName.IsEqual(VariableNameColumnName))
	{
		TreeViewRootObjects.StableSort(
			InColumnSortMode == EColumnSortMode::Ascending ? SortByVariableNameAscending : SortByVariableNameDescending);
	}

	if (bShouldRefreshAfterward)
	{
		FindVisibleObjectsAndRequestTreeRefresh();
	}
}

void SConsoleVariablesEditorList::SetSortOrder(const bool bShouldRefreshAfterward)
{
	for (int32 RowItr = 0; RowItr < TreeViewRootObjects.Num(); RowItr++)
	{
		const TSharedPtr<FConsoleVariablesEditorListRow>& ChildRow = TreeViewRootObjects[RowItr];
		ChildRow->SetSortOrder(RowItr);
	}

	ExecuteSort(CustomSortOrderColumnName, CycleSortMode(CustomSortOrderColumnName), bShouldRefreshAfterward);
}

TSharedPtr<SHeaderRow> SConsoleVariablesEditorList::GenerateHeaderRow()
{
	check(HeaderRow);
	HeaderRow->ClearColumns();

	HeaderRow->AddColumn(
		SHeaderRow::Column(CustomSortOrderColumnName)
			.DefaultLabel(FText::FromString("#"))
			.ToolTipText(LOCTEXT("ClickToSort","Click to sort"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Center)
			.FillWidth(0.4f)
			.ShouldGenerateWidget(true)
			.SortMode_Raw(this, &SConsoleVariablesEditorList::GetSortModeForColumn, CustomSortOrderColumnName)
			.OnSort_Raw(this, &SConsoleVariablesEditorList::OnSortColumnCalled)
	);
	
	HeaderRow->AddColumn(
		SHeaderRow::Column(CheckBoxColumnName)
			.DefaultLabel(LOCTEXT("ConsoleVariablesEditorList_ConsoleVariableCheckboxHeaderText", "Checkbox"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Center)
			.FixedWidth(25.f)
			.ShouldGenerateWidget(true)
			.HeaderContent()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]()
				{
					return HeaderCheckBoxState;
				})
				.OnCheckStateChanged_Lambda([this] (const ECheckBoxState NewState)
				{
					HeaderCheckBoxState = NewState;
					
					for (const FConsoleVariablesEditorListRowPtr& Object : TreeViewRootObjects)
					{
						if (Object->GetCommandInfo().Pin()->ObjectType ==
							FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable)
						{
							Object->SetWidgetCheckedState(NewState);
							if (NewState == ECheckBoxState::Unchecked)
							{
								Object->ResetToStartupValueAndSource();
							}
							else if (NewState == ECheckBoxState::Checked)
							{
								Object->ResetToPresetValue();
							}
						}
					}
				})
			]
	);

	HeaderRow->AddColumn(
		SHeaderRow::Column(VariableNameColumnName)
			.DefaultLabel(LOCTEXT("ConsoleVariablesEditorList_ConsoleVariableNameHeaderText", "Console Variable Name"))
			.ToolTipText(LOCTEXT("ClickToSort","Click to sort"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Left)
			.FillWidth(1.7f)
			.ShouldGenerateWidget(true)
			.SortMode_Raw(this, &SConsoleVariablesEditorList::GetSortModeForColumn, VariableNameColumnName)
			.OnSort_Raw(this, &SConsoleVariablesEditorList::OnSortColumnCalled)
	);

	HeaderRow->AddColumn(
		SHeaderRow::Column(ValueColumnName)
			.DefaultLabel(LOCTEXT("ConsoleVariablesEditorList_ConsoleVariableValueHeaderText", "Value"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Left)
			.ShouldGenerateWidget(true)
			.FillWidth(0.8f)
	);

	HeaderRow->AddColumn(
		SHeaderRow::Column(SourceColumnName)
			.DefaultLabel(LOCTEXT("ConsoleVariablesEditorList_SourceHeaderText", "Source"))
			.ToolTipText(LOCTEXT("ClickToSort","Click to sort"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Left)
			.SortMode_Raw(this, &SConsoleVariablesEditorList::GetSortModeForColumn, SourceColumnName)
			.OnSort_Raw(this, &SConsoleVariablesEditorList::OnSortColumnCalled)
			);
	
	HeaderRow->AddColumn(
		SHeaderRow::Column(ActionButtonColumnName)
			.DefaultLabel(LOCTEXT("ConsoleVariablesEditorList_ConsoleVariableActionButtonHeaderText", "Action"))
			.HAlignHeader(EHorizontalAlignment::HAlign_Center)
			.FixedWidth(25.f)
			.ShouldGenerateWidget(true)
			.HeaderContent()
			[
				SNew(SScaleBox)
				[
					SNew(SImage)
						.Image(this, &SConsoleVariablesEditorList::GetActionColumnImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
	);

	return HeaderRow;
}

void SConsoleVariablesEditorList::SetupFilters()
{
	TArray<EConsoleVariableFlags> SourceFilterTypes =
	{
		ECVF_SetByConstructor,
		ECVF_SetByScalability,
		ECVF_SetByGameSetting,
		ECVF_SetByProjectSetting,
		ECVF_SetBySystemSettingsIni,
		ECVF_SetByDeviceProfile,
		ECVF_SetByGameOverride,
		ECVF_SetByConsoleVariablesIni,
		ECVF_SetByCommandline,
		ECVF_SetByCode,
		ECVF_SetByConsole,
		ECVF_SetByPreview
	};

	for (const EConsoleVariableFlags Type : SourceFilterTypes)
	{
		ShowFilters.Add(MakeShared<FConsoleVariablesEditorListFilter_Source>(Type));
	}

	// Add Show Only Modified filter
	ShowFilters.Add(MakeShared<ConsoleVariablesEditorListFilter_ShowOnlyModifiedVariables>());

	// Add Show Only Set By Current Preset filter
	ShowFilters.Add(MakeShared<ConsoleVariablesEditorListFilter_SetInSession>());

	// Add Show Only Variables filter
	ShowFilters.Add(MakeShared<ConsoleVariablesEditorListFilter_ShowOnlyVariables>());

	// Add Show Only Commands filter
	ShowFilters.Add(MakeShared<ConsoleVariablesEditorListFilter_ShowOnlyCommands>());
}

TSharedRef<SWidget> SConsoleVariablesEditorList::BuildShowOptionsMenu()
{
	FMenuBuilder ShowOptionsMenuBuilder = FMenuBuilder(true, nullptr);

	ShowOptionsMenuBuilder.BeginSection("", LOCTEXT("ShowOptions_ShowSectionHeading", "Show"));
	{
		// Add show filters
		auto AddFiltersLambda = [this, &ShowOptionsMenuBuilder](const TSharedRef<IConsoleVariablesEditorListFilter>& InFilter)
		{
			const FString& FilterName = InFilter->GetFilterName();
			
			ShowOptionsMenuBuilder.AddMenuEntry(
			   InFilter->GetFilterButtonLabel(),
			   InFilter->GetFilterButtonToolTip(),
			   FSlateIcon(),
			   FUIAction(
				   FExecuteAction::CreateLambda(
				   	[this, FilterName]()
					   {
						   ToggleFilterActive(FilterName);
					   }
					),
				   FCanExecuteAction(),
				   FIsActionChecked::CreateSP( InFilter, &IConsoleVariablesEditorListFilter::GetIsFilterActive )
			   ),
			   NAME_None,
			   EUserInterfaceActionType::ToggleButton
		   );
		};

		for (const TSharedRef<IConsoleVariablesEditorListFilter>& Filter : ShowFilters)
		{
			AddFiltersLambda(Filter);
		}
	}
	ShowOptionsMenuBuilder.EndSection();
	
	ShowOptionsMenuBuilder.BeginSection("", LOCTEXT("ShowOptions_SortSectionHeading", "Sort"));
	{
		ShowOptionsMenuBuilder.AddMenuEntry(
			LOCTEXT("SetSortOrder", "Set Sort Order"),
			LOCTEXT("ConsoleVariablesEditorList_SetSortOrder_Tooltip", "Makes the current order of the variables list the saved order."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateRaw(this, &SConsoleVariablesEditorList::SetSortOrder, true)),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	ShowOptionsMenuBuilder.EndSection();
	
	ShowOptionsMenuBuilder.BeginSection("", LOCTEXT("ShowOptions_OptionsSectionHeading", "Options"));
	{
		ShowOptionsMenuBuilder.AddMenuEntry(
			LOCTEXT("TrackAllVariableChanges", "Track All Variable Changes"),
			LOCTEXT("ConsoleVariablesEditorList_TrackAllVariableChanges_Tooltip", "When variables are changed outside the Console Variables Editor, this option will add the variables to the current preset. Does not apply to console commands like 'r.SetNearClipPlane' or 'stat fps'."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([]()
				{
					if (UConsoleVariablesEditorProjectSettings* ProjectSettingsPtr =
						GetMutableDefault<UConsoleVariablesEditorProjectSettings>())
					{
						ProjectSettingsPtr->bAddAllChangedConsoleVariablesToCurrentPreset =
							!ProjectSettingsPtr->bAddAllChangedConsoleVariablesToCurrentPreset;
					}
				}),
			   FCanExecuteAction(),
			   FIsActionChecked::CreateLambda( []()
				{
					if (const UConsoleVariablesEditorProjectSettings* ProjectSettingsPtr =
						GetMutableDefault<UConsoleVariablesEditorProjectSettings>())
					{
						return ProjectSettingsPtr->bAddAllChangedConsoleVariablesToCurrentPreset;
					}

			   		return false;
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	ShowOptionsMenuBuilder.EndSection();

	return ShowOptionsMenuBuilder.MakeWidget();
}

void SConsoleVariablesEditorList::FlushMemory(const bool bShouldKeepMemoryAllocated)
{
	if (bShouldKeepMemoryAllocated)
	{
		TreeViewRootObjects.Reset();
	}
	else
	{
		TreeViewRootObjects.Empty();
	}
}

void SConsoleVariablesEditorList::SetAllGroupsCollapsed()
{
	if (TreeViewPtr.IsValid())
	{
		for (const FConsoleVariablesEditorListRowPtr& RootRow : TreeViewRootObjects)
		{
			if (!RootRow.IsValid())
			{
				continue;
			}
			
			TreeViewPtr->SetItemExpansion(RootRow, false);
			RootRow->SetIsTreeViewItemExpanded(false);
		}
	}
}

void SConsoleVariablesEditorList::OnListViewSearchTextChanged(const FText& Text)
{
	ExecuteListViewSearchOnAllRows(Text.ToString(), true);
}

void SConsoleVariablesEditorList::CacheCurrentListItemData()
{
	const TArray<FConsoleVariablesEditorListRowPtr>& ListItems =
		GetListModelPtr().Pin()->GetListMode() ==
			FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset ? TreeViewRootObjects : LastPresetObjects;

	CachedCommandStates.Empty(CachedCommandStates.Num());

	for (const TSharedPtr<FConsoleVariablesEditorListRow>& Item : ListItems)
	{
		if (const TSharedPtr<FConsoleVariablesEditorCommandInfo>& CommandInfo = Item->GetCommandInfo().Pin())
		{
			const FString& CommandName = CommandInfo->Command;
			
			CachedCommandStates.Add(
				{
					CommandInfo->Command,
					Item->GetCachedValue(),
					Item->GetWidgetCheckedState()
				}
			);
		}
	}
}

void SConsoleVariablesEditorList::RestorePreviousListItemData()
{
	for (const TSharedPtr<FConsoleVariablesEditorListRow>& Item : TreeViewRootObjects)
	{
		if (const TSharedPtr<FConsoleVariablesEditorCommandInfo>& CommandInfo = Item->GetCommandInfo().Pin())
		{
			FConsoleVariablesEditorCommandInfo::EConsoleObjectType CommandType = CommandInfo->ObjectType;
			
			if (CommandInfo->GetConsoleObjectPtr())
			{
				const FString& CommandName = CommandInfo->Command;
				
				if (const FConsoleVariablesEditorAssetSaveData* Match = Algo::FindByPredicate(
					CachedCommandStates,
					[CommandName](const FConsoleVariablesEditorAssetSaveData& CachedData)
					{
						return CachedData.CommandName.Equals(CommandName, ESearchCase::IgnoreCase);
					}))
				{
					Item->SetCachedValue((*Match).CommandValueAsString);
					Item->SetWidgetCheckedState((*Match).CheckedState);
				}
			}
		}
	}
}

void SConsoleVariablesEditorList::GenerateTreeView(const bool bSkipExecutionOfCachedCommands)
{	
	if (!ensure(TreeViewPtr.IsValid()))
	{
		return;
	}
	
	FlushMemory(true);

	const bool bIsPresetMode =
		ListModelPtr.Pin()->GetListMode() == FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset;

	FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();

	TObjectPtr<UConsoleVariablesAsset> EditableAsset = nullptr;

	switch (ListModelPtr.Pin()->GetListMode()) 
	{
		case FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset:
			EditableAsset = ConsoleVariablesEditorModule.GetPresetAsset();
			break;

		case FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::GlobalSearch:
			EditableAsset = ConsoleVariablesEditorModule.GetGlobalSearchAsset();
			break;

		default:
			checkNoEntry();
	}
	
	check(EditableAsset);

	TArray<FConsoleVariablesEditorAssetSaveData> SavedCommands = EditableAsset->GetSavedCommands();
	
	for (const FConsoleVariablesEditorAssetSaveData& SavedCommand : SavedCommands)
	{
		UE_LOGF(LogConsoleVariablesEditor, VeryVerbose, "%s: Considering %ls for the list",
			__FUNCTION__, *SavedCommand.CommandName);
		
		// Get corresponding CommandInfo for tracking or make one if the command is non-value
		TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo =
			ConsoleVariablesEditorModule.FindCommandInfoByName(SavedCommand.CommandName).Pin();

		// Make new CommandInfo if one doesn't exist
		if (!CommandInfo.IsValid())
		{
			CommandInfo = MakeShared<FConsoleVariablesEditorCommandInfo>(SavedCommand.CommandName);

			FConsoleVariablesEditorModule::Get().AddConsoleObjectCommandInfoToMainReference(
				CommandInfo.ToSharedRef());
		}
		
		if (CommandInfo.IsValid())
		{
			FConsoleVariablesEditorListRowPtr RowToAdd = nullptr;
			
			// If the row already exists, let's not make another one.
			if (Algo::FindByPredicate(
				TreeViewRootObjects,
				[this, &CommandInfo](const FConsoleVariablesEditorListRowPtr& ExistingRow)
				{
					return ExistingRow->GetCommandInfo().IsValid() &&
						ExistingRow->GetCommandInfo().Pin()->Command.Equals(CommandInfo->Command, ESearchCase::IgnoreCase);
				}) != nullptr)
			{
				UE_LOGF(LogConsoleVariablesEditor, VeryVerbose, "%s: Not Adding %ls to the list because it's already in the list",
					__FUNCTION__, *SavedCommand.CommandName);
				continue;
			}
			// If the row exists in our cache, let's use that one instead of creating a new one.
			if (TSharedPtr<FConsoleVariablesEditorListRow>* MatchingLastPresetRow = Algo::FindByPredicate(
				LastPresetObjects,
				[this, &CommandInfo](const FConsoleVariablesEditorListRowPtr& ExistingRow)
				{
					return ExistingRow->GetCommandInfo().IsValid() &&
						ExistingRow->GetCommandInfo().Pin()->Command.Equals(CommandInfo->Command, ESearchCase::IgnoreCase);
				}))
			{
				UE_LOGF(LogConsoleVariablesEditor, VeryVerbose, "%s: Adding %ls to the list from cache",
					__FUNCTION__, *SavedCommand.CommandName);
				RowToAdd = *MatchingLastPresetRow;
			}
			else
			{
				const ECheckBoxState NewCheckedState =
					SavedCommand.CheckedState == ECheckBoxState::Unchecked ?
							ECheckBoxState::Unchecked : ECheckBoxState::Checked;
		
				RowToAdd = 
				   MakeShared<FConsoleVariablesEditorListRow>(
						   CommandInfo, SavedCommand.CommandValueAsString,
						   FConsoleVariablesEditorListRow::SingleCommand, 
						   NewCheckedState, SharedThis(this), TreeViewRootObjects.Num(), nullptr);
			}

			if (RowToAdd)
			{
				bool bIsEligibleForExecution =
					bIsPresetMode &&
					CommandInfo->ObjectType != FConsoleVariablesEditorCommandInfo::EConsoleObjectType::NullObject &&
					RowToAdd->IsRowChecked() &&
					!SavedCommand.CommandValueAsString.IsEmpty() &&
					CommandInfo->IsCurrentValueDifferentFromInputValue(SavedCommand.CommandValueAsString);
				
				if (bIsEligibleForExecution && bSkipExecutionOfCachedCommands)
				{
					// If we want to skip execution of cached commands, we need to check the cached commands for a match
					// first. If one isn't found, then we'll execute.
					bIsEligibleForExecution =
						!CachedCommandStates.ContainsByPredicate(
							[this, &CommandInfo](const FConsoleVariablesEditorAssetSaveData& Comparator)
							{
								return Comparator.CommandName.Equals(CommandInfo->Command, ESearchCase::IgnoreCase);
							});
				}
				
				if (bIsEligibleForExecution)
				{
					CommandInfo->ExecuteCommand(SavedCommand.CommandValueAsString);
				}
				
				UE_LOGF(LogConsoleVariablesEditor, VeryVerbose, "%s: Adding %ls to the list with a new row item",
					__FUNCTION__, *SavedCommand.CommandName);
				
				TreeViewRootObjects.Add(RowToAdd);
			}
		}
	}

	// Now clear out the last preset cache if the list is in preset mode
	if (bIsPresetMode)
	{
		LastPresetObjects.Empty(LastPresetObjects.Num());
	}

	TreeViewPtr->RequestTreeRefresh();
}

void SConsoleVariablesEditorList::FindVisibleTreeViewObjects()
{
	VisibleTreeViewObjects.Empty();

	for (const TSharedPtr<FConsoleVariablesEditorListRow>& Row : TreeViewRootObjects)
	{
		if (Row->ShouldBeVisible())
		{
			VisibleTreeViewObjects.Add(Row);
		}
	}
}

void SConsoleVariablesEditorList::FindVisibleObjectsAndRequestTreeRefresh()
{
	FindVisibleTreeViewObjects();
	TreeViewPtr->RequestTreeRefresh();
}

void SConsoleVariablesEditorList::OnGetRowChildren(FConsoleVariablesEditorListRowPtr Row, TArray<FConsoleVariablesEditorListRowPtr>& OutChildren) const
{
	if (Row.IsValid())
	{
		OutChildren = Row->GetChildRows();

		if (Row->GetShouldExpandAllChildren())
		{
			SetChildExpansionRecursively(Row, true);
			Row->SetShouldExpandAllChildren(false);
		}
	}
}

void SConsoleVariablesEditorList::OnRowChildExpansionChange(FConsoleVariablesEditorListRowPtr Row, const bool bIsExpanded, const bool bIsRecursive) const
{
	if (Row.IsValid())
	{
		if (bIsRecursive)
		{
			if (bIsExpanded)
			{
				if (Row->GetRowType() != FConsoleVariablesEditorListRow::HeaderRow)
				{
					Row->SetShouldExpandAllChildren(true);
				}
			}
			else
			{
				SetChildExpansionRecursively(Row, bIsExpanded);
			}
		}
		
		TreeViewPtr->SetItemExpansion(Row, bIsExpanded);
		Row->SetIsTreeViewItemExpanded(bIsExpanded);
	}
}

void SConsoleVariablesEditorList::SetChildExpansionRecursively(const FConsoleVariablesEditorListRowPtr& InRow, const bool bNewIsExpanded) const
{
	if (InRow.IsValid())
	{
		for (const FConsoleVariablesEditorListRowPtr& Child : InRow->GetChildRows())
		{
			TreeViewPtr->SetItemExpansion(Child, bNewIsExpanded);
			Child->SetIsTreeViewItemExpanded(bNewIsExpanded);

			SetChildExpansionRecursively(Child, bNewIsExpanded);
		}
	}
}


const FSlateBrush* SConsoleVariablesEditorList::GetActionColumnImage() const
{
	const FConsoleVariablesEditorList::EConsoleVariablesEditorListMode ListMode =
		ListModelPtr.IsValid()
		? ListModelPtr.Pin()->GetListMode()
		: FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset;

	if (ListMode == FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::GlobalSearch)
	{
		return FAppStyle::Get().GetBrush("Icons.Star");
	}

	return FAppStyle::Get().GetBrush("Icons.Delete");
}

FReply SConsoleVariablesEditorList::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.IsControlDown())
	{
		if (InKeyEvent.GetKey() == EKeys::C)
		{
			CopySelectedNamesAndValues();
			FlashSelectedRows();
			return FReply::Handled();
		}
		if (InKeyEvent.GetKey() == EKeys::V)
		{
			PasteNameAndValue();
			return FReply::Handled();
		}
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SConsoleVariablesEditorList::FlashSelectedRows()
{
	for (const FConsoleVariablesEditorListRowPtr& Row : TreeViewPtr->GetSelectedItems())
	{
		if (Row.IsValid())
		{
			Row->RequestFlash();
		}
	}
}

void SConsoleVariablesEditorList::FlashRowByCommand(const FString& InCommandName)
{
	for (const FConsoleVariablesEditorListRowPtr& Row : TreeViewRootObjects)
	{
		if (!Row.IsValid())
		{
			continue;
		}
		const TSharedPtr<FConsoleVariablesEditorCommandInfo> RowCommand = Row->GetCommandInfo().Pin();
		if (RowCommand.IsValid() && RowCommand->Command.Equals(InCommandName, ESearchCase::IgnoreCase))
		{
			Row->RequestFlash();
			break;
		}
	}
}

void SConsoleVariablesEditorList::CopySelectedVariableNames()
{
	const TArray<FConsoleVariablesEditorListRowPtr> SelectedItems = TreeViewPtr->GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		return;
	}

	FString ClipboardText;
	for (const FConsoleVariablesEditorListRowPtr& Row : SelectedItems)
	{
		if (!Row.IsValid())
		{
			continue;
		}

		const TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo = Row->GetCommandInfo().Pin();
		if (!CommandInfo.IsValid())
		{
			continue;
		}

		if (!ClipboardText.IsEmpty())
		{
			ClipboardText += TEXT("\n");
		}
		ClipboardText += CommandInfo->Command;
	}

	if (!ClipboardText.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
	}
}

void SConsoleVariablesEditorList::CopySelectedNamesAndValues()
{
	const TArray<FConsoleVariablesEditorListRowPtr> SelectedItems = TreeViewPtr->GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		return;
	}

	FString ClipboardText;
	for (const FConsoleVariablesEditorListRowPtr& Row : SelectedItems)
	{
		if (!Row.IsValid())
		{
			continue;
		}

		const TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo = Row->GetCommandInfo().Pin();
		if (!CommandInfo.IsValid())
		{
			continue;
		}

		FString Value;
		if (!CommandInfo->GetCurrentValueAsString(Value))
		{
			Value = Row->GetCachedValue();
		}

		if (!ClipboardText.IsEmpty())
		{
			ClipboardText += TEXT("\n");
		}
		ClipboardText += FString::Printf(TEXT("%s %s"), *CommandInfo->Command, *Value);
	}

	if (!ClipboardText.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
	}
}

void SConsoleVariablesEditorList::CopySelectedValues()
{
	const TArray<FConsoleVariablesEditorListRowPtr> SelectedItems = TreeViewPtr->GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		return;
	}

	FString ClipboardText;
	for (const FConsoleVariablesEditorListRowPtr& Row : SelectedItems)
	{
		if (!Row.IsValid())
		{
			continue;
		}

		const TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo = Row->GetCommandInfo().Pin();
		if (!CommandInfo.IsValid())
		{
			continue;
		}

		FString Value;
		if (!CommandInfo->GetCurrentValueAsString(Value))
		{
			Value = Row->GetCachedValue();
		}

		if (!ClipboardText.IsEmpty())
		{
			ClipboardText += TEXT("\n");
		}
		ClipboardText += Value;
	}

	if (!ClipboardText.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
	}
}

void SConsoleVariablesEditorList::PasteValueToSelectedVariable()
{
	const TArray<FConsoleVariablesEditorListRowPtr> SelectedItems = TreeViewPtr->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return;
	}

	const FConsoleVariablesEditorListRowPtr& Row = SelectedItems[0];
	if (!Row.IsValid() || !Row->IsRowChecked())
	{
		return;
	}

	const TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo = Row->GetCommandInfo().Pin();
	if (!CommandInfo.IsValid() ||
		CommandInfo->ObjectType != FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable)
	{
		return;
	}

	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	ClipboardText = ClipboardText.TrimStartAndEnd();

	if (ClipboardText.IsEmpty())
	{
		return;
	}

	CommandInfo->ExecuteCommand(ClipboardText);
	Row->SetCachedValue(ClipboardText);
	Row->RequestFlash();
	FindVisibleObjectsAndRequestTreeRefresh();
}

void SConsoleVariablesEditorList::PasteNameAndValue()
{
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	if (ClipboardText.IsEmpty())
	{
		return;
	}

	TArray<FString> Lines;
	ClipboardText.ParseIntoArrayLines(Lines);

	bool bAppliedAny = false;

	for (const FString& Line : Lines)
	{
		const FString TrimmedLine = Line.TrimStartAndEnd();
		if (TrimmedLine.IsEmpty())
		{
			continue;
		}

		FString CommandName;
		FString Value;
		if (!TrimmedLine.Split(TEXT(" "), &CommandName, &Value))
		{
			UE_LOG(LogConsoleVariablesEditor, Error,
				TEXT("Paste Name and Value: Failed to parse '%s' as a console variable command. Expected format: 'VariableName Value'"),
				*TrimmedLine);
			continue;
		}

		CommandName = CommandName.TrimStartAndEnd();
		Value = Value.TrimStartAndEnd();

		if (CommandName.IsEmpty() || Value.IsEmpty())
		{
			UE_LOG(LogConsoleVariablesEditor, Error,
				TEXT("Paste Name and Value: Failed to parse '%s' as a console variable command. Expected format: 'VariableName Value'"),
				*TrimmedLine);
			continue;
		}

		TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo =
			FConsoleVariablesEditorModule::Get().FindCommandInfoByName(CommandName).Pin();

		if (!CommandInfo.IsValid())
		{
			UE_LOG(LogConsoleVariablesEditor, Error,
				TEXT("Paste Name and Value: Console variable '%s' not found."),
				*CommandName);
			continue;
		}

		CommandInfo->ExecuteCommand(Value);
		bAppliedAny = true;

		bool bFoundExistingRow = false;
		for (const FConsoleVariablesEditorListRowPtr& Row : TreeViewRootObjects)
		{
			if (Row.IsValid() && Row->GetCommandInfo().IsValid())
			{
				const TSharedPtr<FConsoleVariablesEditorCommandInfo> RowCommand = Row->GetCommandInfo().Pin();
				if (RowCommand.IsValid() && RowCommand->Command.Equals(CommandName, ESearchCase::IgnoreCase))
				{
					Row->SetCachedValue(Value);
					Row->RequestFlash();
					bFoundExistingRow = true;
					break;
				}
			}
		}

		// If the variable isn't already in the preset list, add it. The list will rebuild and
		// the new row will flash on scroll-into-view via SetShouldFlashOnScrollIntoView.
		if (!bFoundExistingRow)
		{
			const FString CommandLine = FString::Printf(TEXT("%s %s"), *CommandName, *Value);
			FConsoleVariablesEditorModule::Get().ValidateConsoleInputAndAddToCurrentPreset(
				FText::FromString(CommandLine));
		}
	}

	if (bAppliedAny)
	{
		FindVisibleObjectsAndRequestTreeRefresh();
	}
}

TSharedPtr<SWidget> SConsoleVariablesEditorList::OnTreeViewContextMenuOpening()
{
	return BuildContextMenuWidget();
}

TSharedRef<SWidget> SConsoleVariablesEditorList::BuildContextMenuWidget()
{
	const TArray<FConsoleVariablesEditorListRowPtr> SelectedItems = TreeViewPtr->GetSelectedItems();
	const bool bHasSelection = SelectedItems.Num() > 0;

	bool bCanPasteValue = false;
	if (SelectedItems.Num() == 1)
	{
		const FConsoleVariablesEditorListRowPtr& Row = SelectedItems[0];
		if (Row.IsValid() && Row->IsRowChecked())
		{
			const TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo = Row->GetCommandInfo().Pin();
			bCanPasteValue = CommandInfo.IsValid() &&
				CommandInfo->ObjectType == FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable;
		}
	}

	const TAttribute<FText> CopyValueChord =
		FInputChord(EModifierKey::Shift, EKeys::RightMouseButton).GetInputText(false);
	const TAttribute<FText> PasteValueChord =
		FInputChord(EModifierKey::Shift, EKeys::LeftMouseButton).GetInputText(false);
	const TAttribute<FText> CopyNameAndValueChord =
		FInputChord(EModifierKey::Control, EKeys::C).GetInputText(false);
	const TAttribute<FText> PasteNameAndValueChord =
		FInputChord(EModifierKey::Control, EKeys::V).GetInputText(false);

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("Edit", LOCTEXT("EditSectionHeading", "Edit"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyValue", "Copy Value"),
			LOCTEXT("CopyValueTooltip", "Copy the selected console variable value(s) to the clipboard"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					CopySelectedValues();
					FlashSelectedRows();
				}),
				FCanExecuteAction::CreateLambda([bHasSelection]() { return bHasSelection; })
			),
			NAME_None,
			EUserInterfaceActionType::Button,
			NAME_None,
			CopyValueChord
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("PasteValue", "Paste Value"),
			LOCTEXT("PasteValueTooltip", "Paste the clipboard value into the selected console variable"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SConsoleVariablesEditorList::PasteValueToSelectedVariable),
				FCanExecuteAction::CreateLambda([bCanPasteValue]() { return bCanPasteValue; })
			),
			NAME_None,
			EUserInterfaceActionType::Button,
			NAME_None,
			PasteValueChord
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyName", "Copy Console Variable Name"),
			LOCTEXT("CopyNameTooltip", "Copy the selected console variable name(s) to the clipboard"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					CopySelectedVariableNames();
					FlashSelectedRows();
				}),
				FCanExecuteAction::CreateLambda([bHasSelection]() { return bHasSelection; })
			)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyNameAndValue", "Copy Console Variable Name And Value"),
			LOCTEXT("CopyNameAndValueTooltip", "Copy the selected console variable(s) as 'Name Value' to the clipboard"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					CopySelectedNamesAndValues();
					FlashSelectedRows();
				}),
				FCanExecuteAction::CreateLambda([bHasSelection]() { return bHasSelection; })
			),
			NAME_None,
			EUserInterfaceActionType::Button,
			NAME_None,
			CopyNameAndValueChord
		);

		// Paste Name and Value is always enabled — it can target an existing row OR add a new
		// variable to the preset, including when the list is empty or the user right-clicks
		// the table background.
		MenuBuilder.AddMenuEntry(
			LOCTEXT("PasteNameAndValue", "Paste Name and Value"),
			LOCTEXT("PasteNameAndValueTooltip", "Paste 'Name Value' from the clipboard and apply, adding the variable to the preset if it isn't already present"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SConsoleVariablesEditorList::PasteNameAndValue)),
			NAME_None,
			EUserInterfaceActionType::Button,
			NAME_None,
			PasteNameAndValueChord
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FReply SConsoleVariablesEditorList::OnEmptyAreaMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::RightMouseButton)
	{
		return FReply::Unhandled();
	}

	const TSharedRef<SWidget> MenuWidget = BuildContextMenuWidget();
	FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(
		AsShared(),
		WidgetPath,
		MenuWidget,
		MouseEvent.GetScreenSpacePosition(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
