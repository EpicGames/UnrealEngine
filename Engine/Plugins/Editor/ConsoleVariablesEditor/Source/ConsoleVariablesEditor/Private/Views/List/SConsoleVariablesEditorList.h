// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConsoleVariablesAsset.h"
#include "ConsoleVariablesEditorList.h"
#include "ConsoleVariablesEditorListRow.h"
#include "ConsoleVariablesEditorListFilters/IConsoleVariablesEditorListFilter.h"

#include "Misc/EnumRange.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STreeView.h"

class SConsoleVariablesEditorGlobalSearchToggle;
class SWrapBox;
class FConsoleVariablesEditorList;
class UConsoleVariablesAsset;
class SBorder;
class SBox;
class SComboButton;
class SSearchBox;
class SHeaderRow;

class SConsoleVariablesEditorList final : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS(SConsoleVariablesEditorList)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FConsoleVariablesEditorList> ListModel);

	virtual ~SConsoleVariablesEditorList() override;

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	TWeakPtr<FConsoleVariablesEditorList> GetListModelPtr()
	{
		return ListModelPtr;
	}

	FReply TryEnterGlobalSearch(const FString& SearchString = "");
	FReply HandleRemoveGlobalSearchToggleButton();
	void CleanUpGlobalSearchesMarkedForDelete();
	/** Remove all widgets from the global search container then recreate them */
	void RefreshGlobalSearchWidgets();

	/*
	 * Regenerate the list items and refresh the list. Call when adding or removing variables.
	 * @param bShouldCacheValues If true, the current list's current values will be cached and then restored when the list is rebuilt. Otherwise preset values will be used.
	 */
	void RebuildListWithListMode(FConsoleVariablesEditorList::EConsoleVariablesEditorListMode NewListMode, const FString& InConsoleCommandToScrollTo = "", bool bShouldCacheValues = true);

	/**
	 * Refresh filters and sorting.
	 * Useful for when the list state has gone stale but the variable count has not changed.
	 */
	void RefreshList();

	[[nodiscard]] TArray<FConsoleVariablesEditorListRowPtr> GetSelectedTreeViewItems() const;

	[[nodiscard]] TArray<FConsoleVariablesEditorListRowPtr> GetTreeViewItems() const;
	void SetTreeViewItems(const TArray<FConsoleVariablesEditorListRowPtr>& InItems);

	/** Updates the saved values in a UConsoleVariablesAsset so that the command/value map can be saved to disk */
	void UpdatePresetValuesForSave(const TObjectPtr<UConsoleVariablesAsset> InAsset) const;

	/** Update the cached value stored for the given command. */
	void UpdateCachedValue(const FString& InCommand, const FString& InValue);

	FString GetSearchStringFromSearchInputField() const;
	void SetSearchStringInSearchInputField(const FString InSearchString) const;
	void ExecuteListViewSearchOnAllRows(const FString& SearchString, const bool bShouldRefreshAfterward = true);

	bool DoesTreeViewHaveVisibleChildren() const;

	void SetTreeViewItemExpanded(const TSharedPtr<FConsoleVariablesEditorListRow>& RowToExpand, const bool bNewExpansion) const;

	void SetAllListViewItemsCheckState(const ECheckBoxState InNewState);

	bool DoesListHaveCheckedMembers() const;

	bool DoesListHaveUncheckedMembers() const;
	
	void OnListItemCheckBoxStateChange(const ECheckBoxState InNewState);

	void ToggleFilterActive(const FString& FilterName);
	void EvaluateIfRowsPassFilters(const bool bShouldRefreshAfterward = true);

	// Sorting

	const FName& GetActiveSortingColumnName() const
	{
		return ActiveSortingColumnName;
	}
	EColumnSortMode::Type GetSortModeForColumn(FName InColumnName) const;
	void OnSortColumnCalled(EColumnSortPriority::Type Priority, const FName& ColumnName, EColumnSortMode::Type SortMode);
	EColumnSortMode::Type CycleSortMode(const FName& InColumnName);
	void ExecuteSort(
		const FName& InColumnName, const EColumnSortMode::Type InColumnSortMode, const bool bShouldRefreshAfterward = true);
	void ClearSorting()
	{
		ActiveSortingColumnName = NAME_None;
		ActiveSortingType = EColumnSortMode::None;
	}
	void SetSortOrder(const bool bShouldRefreshAfterward = true);

	// Copy/Paste — public so row widgets can invoke on Shift+RMB / Shift+LMB

	void CopySelectedVariableNames();
	void CopySelectedNamesAndValues();
	void CopySelectedValues();
	void PasteValueToSelectedVariable();
	void PasteNameAndValue();

	// Column Names

	static const FName CustomSortOrderColumnName;
	static const FName CheckBoxColumnName;
	static const FName VariableNameColumnName;
	static const FName ValueColumnName;
	static const FName SourceColumnName;
	static const FName ActionButtonColumnName;

private:

	TWeakPtr<FConsoleVariablesEditorList> ListModelPtr;
	
	TSharedPtr<SHeaderRow> HeaderRow;
	TSharedPtr<SHeaderRow> GenerateHeaderRow();
	ECheckBoxState HeaderCheckBoxState = ECheckBoxState::Checked;

	void SetupFilters();

	TSharedRef<SWidget> BuildShowOptionsMenu();
	
	void FlushMemory(const bool bShouldKeepMemoryAllocated);
	
	void SetAllGroupsCollapsed();

	// Search
	
	void OnListViewSearchTextChanged(const FText& Text);

	TSharedPtr<SSearchBox> ListSearchBoxPtr;
	TSharedPtr<SComboButton> ViewOptionsComboButton;
	
	/** Contains the GlobalSearchesContainer and remove button. Made a member in order to collapse it when GlobalSearchesContainer has no children. */
	TSharedPtr<SHorizontalBox> GlobalSearchesHBox;
	/** Contains the global search checkboxes only */
	TSharedPtr<SWrapBox> GlobalSearchesContainer;
	TArray<TSharedRef<SConsoleVariablesEditorGlobalSearchToggle>> CurrentGlobalSearches;
	
	TSharedPtr<SCheckBox> RemoveGlobalSearchesButtonPtr;
	TSharedPtr<SBox> ListBoxContainerPtr;

	// Copy/Paste

	TSharedPtr<SWidget> OnTreeViewContextMenuOpening();
	TSharedRef<SWidget> BuildContextMenuWidget();
	FReply OnEmptyAreaMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Flash all currently selected rows, e.g. after a Ctrl+C / Ctrl+V hotkey. */
	void FlashSelectedRows();
	/** Flash a row whose command matches InCommandName (case-insensitive). */
	void FlashRowByCommand(const FString& InCommandName);

	TSharedPtr<SBorder> EmptyAreaBorder;

	//  Tree View Implementation

	void CacheCurrentListItemData();
	void RestorePreviousListItemData();
	void GenerateTreeView(const bool bSkipExecutionOfCachedCommands = true);
	void FindVisibleTreeViewObjects();
	void FindVisibleObjectsAndRequestTreeRefresh();
	
	void OnGetRowChildren(FConsoleVariablesEditorListRowPtr Row, TArray<FConsoleVariablesEditorListRowPtr>& OutChildren) const;
	void OnRowChildExpansionChange(FConsoleVariablesEditorListRowPtr Row, const bool bIsExpanded, const bool bIsRecursive = false) const;

	void SetChildExpansionRecursively(const FConsoleVariablesEditorListRowPtr& InRow, const bool bNewIsExpanded) const;

	const FSlateBrush* GetActionColumnImage() const;

	TArray<TSharedRef<IConsoleVariablesEditorListFilter>> ShowFilters;

	TSharedPtr<STreeView<FConsoleVariablesEditorListRowPtr>> TreeViewPtr;

	/** All Tree view objects */
	TArray<FConsoleVariablesEditorListRowPtr> TreeViewRootObjects;
	/** Visible Tree view objects, after filters */
	TArray<FConsoleVariablesEditorListRowPtr> VisibleTreeViewObjects;
	/** When a global search is initiated we cache the tree objects shown in preset list mode
	 * This way they can be recalled without rebuilding from the saved commands
	 */
	TArray<FConsoleVariablesEditorListRowPtr> LastPresetObjects;

	/** A collection of list item data to save and restore when rebuilding the list. */
	TArray<FConsoleVariablesEditorAssetSaveData> CachedCommandStates;

	// Sorting

	FName ActiveSortingColumnName = NAME_None;
	EColumnSortMode::Type ActiveSortingType = EColumnSortMode::None;

	TFunction<bool(const FConsoleVariablesEditorListRowPtr&, const FConsoleVariablesEditorListRowPtr&)> SortByOrderAscending = [](const FConsoleVariablesEditorListRowPtr& A, const FConsoleVariablesEditorListRowPtr& B)
		{
			return A->GetSortOrder() < B->GetSortOrder();
		};

	TFunction<bool(const FConsoleVariablesEditorListRowPtr&, const FConsoleVariablesEditorListRowPtr&)> SortBySourceAscending = [](const FConsoleVariablesEditorListRowPtr& A, const FConsoleVariablesEditorListRowPtr& B)
		{
			return A->GetCommandInfo().Pin()->GetSourceAsText().ToString() < B->GetCommandInfo().Pin()->GetSourceAsText().ToString();
		};

	TFunction<bool(const FConsoleVariablesEditorListRowPtr&, const FConsoleVariablesEditorListRowPtr&)> SortBySourceDescending = [](const FConsoleVariablesEditorListRowPtr& A, const FConsoleVariablesEditorListRowPtr& B)
		{
			return B->GetCommandInfo().Pin()->GetSourceAsText().ToString() < A->GetCommandInfo().Pin()->GetSourceAsText().ToString();
		};

	TFunction<bool(const FConsoleVariablesEditorListRowPtr&, const FConsoleVariablesEditorListRowPtr&)> SortByVariableNameAscending = [](const FConsoleVariablesEditorListRowPtr& A, const FConsoleVariablesEditorListRowPtr& B)
		{
			return A->GetCommandInfo().Pin()->Command < B->GetCommandInfo().Pin()->Command;
		};

	TFunction<bool(const FConsoleVariablesEditorListRowPtr&, const FConsoleVariablesEditorListRowPtr&)> SortByVariableNameDescending = [](const FConsoleVariablesEditorListRowPtr& A, const FConsoleVariablesEditorListRowPtr& B)
		{
			return B->GetCommandInfo().Pin()->Command < A->GetCommandInfo().Pin()->Command;
		};
};
