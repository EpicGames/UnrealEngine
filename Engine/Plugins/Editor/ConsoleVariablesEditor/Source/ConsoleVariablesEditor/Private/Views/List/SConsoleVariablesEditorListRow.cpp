// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConsoleVariablesEditorListRow.h"

#include "ConsoleVariablesEditorList.h"
#include "ConsoleVariablesEditorModule.h"
#include "ConsoleVariablesEditorStyle.h"
#include "MultiUser/ConsoleVariableSync.h"
#include "SConsoleVariablesEditorListValueInput.h"
#include "Views/Widgets/SConsoleVariablesEditorTooltipWidget.h"

#include "Input/DragAndDrop.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

class FConsoleVariablesListRowDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FConsoleVariablesListRowDragDropOp, FDecoratedDragDropOp)

	/** The item being dragged and dropped */
	TArray<FConsoleVariablesEditorListRowPtr> DraggedItems;

	/** Constructs a new drag/drop operation */
	static TSharedRef<FConsoleVariablesListRowDragDropOp> New(const TArray<FConsoleVariablesEditorListRowPtr>& InItems)
	{
		check(InItems.Num() > 0);

		TSharedRef<FConsoleVariablesListRowDragDropOp> Operation = MakeShareable(
			new FConsoleVariablesListRowDragDropOp());

		Operation->DraggedItems = InItems;

		Operation->DefaultHoverIcon = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.Error");

		// Set the display text and the transaction name based on whether we're dragging a single or multiple widgets
		if (InItems.Num() == 1)
		{
			Operation->DefaultHoverText = FText::FromString(InItems[0]->GetCommandInfo().Pin()->Command);
		}
		else
		{
			Operation->DefaultHoverText =
				FText::Format(
					SConsoleVariablesEditorListRow::MultiDragFormatText,
					FText::AsNumber(Operation->DraggedItems.Num())
				);
		}

		Operation->Construct();

		return Operation;
	}
};

void SConsoleVariablesEditorListRow::Construct(
	const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable,
	const TWeakPtr<FConsoleVariablesEditorListRow> InRow)
{
	check(InRow.IsValid());

	Item = InRow;
	const FConsoleVariablesEditorListRowPtr PinnedItem = Item.Pin();
	const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedCommand = PinnedItem->GetCommandInfo().Pin();

	check(PinnedCommand.IsValid());

	// Set up flash animation
	FlashAnimation = FCurveSequence(0.f, FlashAnimationDuration, ECurveEaseFunction::QuadInOut);

	SMultiColumnTableRow<FConsoleVariablesEditorListRowPtr>::Construct(
		FSuperRowType::FArguments()
		.Padding(FMargin(0.0f, 1.0f))
		.OnCanAcceptDrop(this, &SConsoleVariablesEditorListRow::HandleCanAcceptDrop)
		.OnAcceptDrop(this, &SConsoleVariablesEditorListRow::HandleAcceptDrop)
		.OnDragDetected(this, &SConsoleVariablesEditorListRow::HandleDragDetected)
		.OnDragLeave(this, &SConsoleVariablesEditorListRow::HandleDragLeave)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow")),
		InOwnerTable
	);

	if (PinnedItem->GetShouldFlashOnScrollIntoView())
	{
		FlashRow();

		PinnedItem->SetShouldFlashOnScrollIntoView(false);
	}

	PinnedItem->OnFlashRequested.AddSP(this, &SConsoleVariablesEditorListRow::FlashRow);
}

FReply SConsoleVariablesEditorListRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetModifierKeys().IsShiftDown())
	{
		const FConsoleVariablesEditorListRowPtr PinnedItem = Item.Pin();
		const TSharedPtr<SConsoleVariablesEditorList> ListView =
			PinnedItem.IsValid() ? PinnedItem->GetListViewPtr().Pin() : nullptr;
		if (ListView.IsValid())
		{
			if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
			{
				ListView->CopySelectedValues();
				FlashRow();
				return FReply::Handled();
			}
			if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
			{
				ListView->PasteValueToSelectedVariable();
				FlashRow();
				return FReply::Handled();
			}
		}
	}

	return SMultiColumnTableRow<FConsoleVariablesEditorListRowPtr>::OnMouseButtonUp(MyGeometry, MouseEvent);
}

TSharedRef<SWidget> SConsoleVariablesEditorListRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	const FConsoleVariablesEditorListRowPtr PinnedItem = Item.Pin();

	const TSharedPtr<SWidget> CellWidget = GenerateCells(InColumnName, PinnedItem);

	const TSharedRef<SImage> FlashImage = SNew(SImage)
										.Image(new FSlateColorBrush(FStyleColors::White))
										.Visibility_Raw(
											this, &SConsoleVariablesEditorListRow::GetFlashImageVisibility)
										.ColorAndOpacity_Raw(
											this, &SConsoleVariablesEditorListRow::GetFlashImageColorAndOpacity);

	FlashImages.Add(FlashImage);

	return SNew(SOverlay)
			.Visibility(EVisibility::SelfHitTestInvisible)

			+ SOverlay::Slot()
			[
				FlashImage
			]

			+ SOverlay::Slot()
			[
				SNew(SBorder)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.BorderImage(GetBorderImage(PinnedItem->GetRowType()))
				[
					CellWidget.ToSharedRef()
				]
			];
}

SConsoleVariablesEditorListRow::~SConsoleVariablesEditorListRow()
{
	Item.Reset();
	HoverToolTip.Reset();

	FlashImages.Empty();

	ValueChildInputWidget.Reset();

	HoverableWidgetsPtr.Reset();
}

FReply SConsoleVariablesEditorListRow::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TArray<FConsoleVariablesEditorListRowPtr> DraggedItems = Item.Pin()->GetSelectedTreeViewItems();
	TSharedRef<FConsoleVariablesListRowDragDropOp> Operation =
		FConsoleVariablesListRowDragDropOp::New(DraggedItems);

	return FReply::Handled().BeginDragDrop(Operation);
}

void SConsoleVariablesEditorListRow::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FConsoleVariablesListRowDragDropOp> Operation =
		DragDropEvent.GetOperationAs<FConsoleVariablesListRowDragDropOp>())
	{
		Operation->ResetToDefaultToolTip();
	}
}

TOptional<EItemDropZone> SConsoleVariablesEditorListRow::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent,
                                                                             EItemDropZone DropZone,
                                                                             FConsoleVariablesEditorListRowPtr
                                                                             TargetItem)
{
	TSharedPtr<FConsoleVariablesListRowDragDropOp> Operation =
		DragDropEvent.GetOperationAs<FConsoleVariablesListRowDragDropOp>();

	if (!Operation.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	const bool bShouldEvaluateDrop = Item.IsValid() && Item.Pin()->GetListViewPtr().IsValid() &&
		Item.Pin()->GetListViewPtr().Pin()->GetActiveSortingColumnName().IsEqual(
			SConsoleVariablesEditorList::CustomSortOrderColumnName);

	if (!bShouldEvaluateDrop)
	{
		Operation->SetToolTip(
			LOCTEXT("SortByCustomOrderDrgDropWarning", "Sort by custom order (\"#\") to drag & drop"),
			FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.Error")
		);

		return TOptional<EItemDropZone>();
	}

	const bool bIsDropDenied = !TargetItem.IsValid() || Operation->DraggedItems.Num() < 1 ||
		!TargetItem->GetCommandInfo().IsValid() || Operation->DraggedItems.Contains(TargetItem);

	FString BoolAsString = bIsDropDenied ? "true" : "false";

	if (bIsDropDenied)
	{
		Operation->ResetToDefaultToolTip();

		return TOptional<EItemDropZone>();
	}

	FText ItemNameText = FText::FromString(Operation->DraggedItems[0]->GetCommandInfo().Pin()->Command);

	if (Operation->DraggedItems.Num() > 1)
	{
		ItemNameText = FText::Format(MultiDragFormatText, FText::AsNumber(Operation->DraggedItems.Num()));
	}

	const FText DropPermittedText =
		FText::Format(InsertFormatText,
		              ItemNameText,
		              DropZone == EItemDropZone::BelowItem ? BelowText : AboveText,
		              FText::FromString(TargetItem->GetCommandInfo().Pin()->Command)
		);

	Operation->SetToolTip(
		DropPermittedText,
		FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.OK")
	);

	// We have no behaviour yet for dropping one item onto another, so we'll treat it like we dropped it above
	return DropZone == EItemDropZone::OntoItem ? EItemDropZone::AboveItem : DropZone;
}

FReply SConsoleVariablesEditorListRow::HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone,
                                                        FConsoleVariablesEditorListRowPtr TargetItem)
{
	TSharedPtr<FConsoleVariablesListRowDragDropOp> Operation =
		DragDropEvent.GetOperationAs<FConsoleVariablesListRowDragDropOp>();

	if (!Operation.IsValid())
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<SConsoleVariablesEditorList> ListView = Item.Pin()->GetListViewPtr().Pin();

	TArray<FConsoleVariablesEditorListRowPtr> DraggedItems = Operation->DraggedItems;

	TArray<FConsoleVariablesEditorListRowPtr> AllTreeItemsCopy = ListView->GetTreeViewItems();

	for (FConsoleVariablesEditorListRowPtr DraggedItem : DraggedItems)
	{
		if (!DraggedItem.IsValid() || !AllTreeItemsCopy.Contains(DraggedItem))
		{
			continue;
		}

		AllTreeItemsCopy.Remove(DraggedItem);
	}

	const int32 TargetIndex = AllTreeItemsCopy.IndexOfByKey(TargetItem);

	if (TargetIndex > -1)
	{
		for (int32 ItemIndex = DraggedItems.Num() - 1; ItemIndex >= 0; ItemIndex--)
		{
			FConsoleVariablesEditorListRowPtr DraggedItem = DraggedItems[ItemIndex];

			if (!DraggedItem.IsValid() || AllTreeItemsCopy.Contains(DraggedItem))
			{
				continue;
			}

			AllTreeItemsCopy.Insert(DraggedItem, DropZone == EItemDropZone::AboveItem ? TargetIndex : TargetIndex + 1);
		}

		ListView->SetTreeViewItems(AllTreeItemsCopy);
		ListView->SetSortOrder();
	}

	return FReply::Handled();
}

void SConsoleVariablesEditorListRow::FlashRow()
{
	FlashAnimation.Play(this->AsShared());
}

EVisibility SConsoleVariablesEditorListRow::GetHoverableWidgetsVisibility() const
{
	const FConsoleVariablesEditorListRowPtr RowPtr = Item.Pin();

	if (IsHovered() || IsSelected() ||
		(RowPtr.IsValid() && RowPtr->IsGlobalSearch() && RowPtr->IsInPreset()))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility SConsoleVariablesEditorListRow::GetFlashImageVisibility() const
{
	return FlashAnimation.IsPlaying() ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden;
}

FSlateColor SConsoleVariablesEditorListRow::GetFlashImageColorAndOpacity() const
{
	if (FlashAnimation.IsPlaying())
	{
		// This equation modulates the alpha into a parabolic curve 
		const float Progress = FMath::Abs(FMath::Abs((FlashAnimation.GetLerp() - 0.5f) * 2) - 1);
		return FLinearColor::LerpUsingHSV(FLinearColor::Transparent, FlashColor, Progress);
	}

	return FLinearColor::Transparent;
}

const FSlateBrush* SConsoleVariablesEditorListRow::GetBorderImage(
	const FConsoleVariablesEditorListRow::EConsoleVariablesEditorListRowType InRowType)
{
	switch (InRowType)
	{
	case FConsoleVariablesEditorListRow::CommandGroup:
		return FConsoleVariablesEditorStyle::Get().GetBrush("ConsoleVariablesEditor.CommandGroupBorder");

	case FConsoleVariablesEditorListRow::HeaderRow:
		return FConsoleVariablesEditorStyle::Get().GetBrush("ConsoleVariablesEditor.HeaderRowBorder");

	default:
		return FConsoleVariablesEditorStyle::Get().GetBrush("ConsoleVariablesEditor.DefaultBorder");
	}
}

TSharedRef<SWidget> SConsoleVariablesEditorListRow::GenerateCells(const FName& InColumnName,
                                                                  const TSharedPtr<FConsoleVariablesEditorListRow>
                                                                  PinnedItem)
{
	if (InColumnName.IsEqual(SConsoleVariablesEditorList::CustomSortOrderColumnName))
	{
		return SNew(STextBlock)
				.Visibility(EVisibility::SelfHitTestInvisible)
				.Justification(ETextJustify::Center)
				.Text_Lambda([this, PinnedItem]()
		                       {
			                       return FText::AsNumber(PinnedItem->GetSortOrder() + 1);
		                       });
	}
	if (InColumnName.IsEqual(SConsoleVariablesEditorList::CheckBoxColumnName))
	{
		return SNew(SBox)
				.Visibility(EVisibility::SelfHitTestInvisible)
				.HAlign(HAlign_Center)
				.Padding(FMargin(1,0,0,0))
		[
			SNew(SCheckBox)
			.HAlign(HAlign_Center)
			.IsChecked_Raw(this, &SConsoleVariablesEditorListRow::GetCheckboxState)
			.OnCheckStateChanged_Raw(this, &SConsoleVariablesEditorListRow::OnCheckboxStateChange)
			.Visibility(Item.Pin()->GetCommandInfo().Pin()->ObjectType ==
			            FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable
				            ? EVisibility::Visible
				            : EVisibility::Collapsed)
		];
	}
	if (InColumnName.IsEqual(SConsoleVariablesEditorList::VariableNameColumnName))
	{
		if (!HoverToolTip.IsValid())
		{
			HoverToolTip = SConsoleVariablesEditorTooltipWidget::MakeTooltip(
				PinnedItem->GetCommandInfo().Pin()->Command,
				PinnedItem->GetCommandInfo().Pin()->GetHelpText());
		}
		return SNew(SBox)
				.Visibility(EVisibility::SelfHitTestInvisible)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(TextBlockLeftPadding, 0.f, 0.f, 0.f))
				[
					SNew(STextBlock)
					.Visibility(EVisibility::Visible)
					.Justification(ETextJustify::Left)
					.Text(FText::FromString(PinnedItem->GetCommandInfo().Pin()->Command))
					.ToolTip(HoverToolTip)
				];
	}
	if (InColumnName.IsEqual(SConsoleVariablesEditorList::ValueColumnName))
	{
		return GenerateValueCellWidget(PinnedItem);
	}
	if (InColumnName.IsEqual(SConsoleVariablesEditorList::SourceColumnName))
	{
		return SNew(SBox)
				.Visibility(EVisibility::HitTestInvisible)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(TextBlockLeftPadding, 0.f, 0.f, 0.f))
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
	                {
		                return Item.Pin()->GetCommandInfo().Pin()->GetSourceAsText();
	                })
				];
	}
	if (InColumnName.IsEqual(SConsoleVariablesEditorList::ActionButtonColumnName))
	{
		return SNew(SConsoleVariablesEditorListRowHoverWidgets, Item)
				.Visibility(this, &SConsoleVariablesEditorListRow::GetHoverableWidgetsVisibility);
	}

	return SNullWidget::NullWidget;
}

ECheckBoxState SConsoleVariablesEditorListRow::GetCheckboxState() const
{
	//Non-variable types always checked
	if (Item.Pin()->GetCommandInfo().Pin()->ObjectType !=
		FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable)
	{
		return ECheckBoxState::Checked;
	}

	if (ensure(Item.IsValid()))
	{
		return Item.Pin()->GetWidgetCheckedState();
	}

	return ECheckBoxState::Checked;
}

void SConsoleVariablesEditorListRow::OnCheckboxStateChange(const ECheckBoxState InNewState) const
{
	const TSharedPtr<FConsoleVariablesEditorListRow> PinnedItem = Item.Pin();

	if (!PinnedItem.IsValid())
	{
		return;
	}

	for (const FConsoleVariablesEditorListRowPtr& RowPtr : PinnedItem->GetRowsAffectedByActions())
	{
		if (!RowPtr.IsValid())
		{
			continue;
		}

		if (RowPtr->GetCommandInfo().Pin()->ObjectType !=
			FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable)
		{
			// Only variables can be checked/unchecked
			continue;
		}

		RowPtr->SetWidgetCheckedState(InNewState, true);

		if (RowPtr->GetRowType() == FConsoleVariablesEditorListRow::SingleCommand)
		{
			if (RowPtr->IsRowChecked())
			{
				RowPtr->GetCommandInfo().Pin()->ExecuteCommand(RowPtr->GetCachedValue());
			}
			else
			{
				RowPtr->ResetToStartupValueAndSource();
			}
		}
	}

	if (const TWeakPtr<SConsoleVariablesEditorList> ListView = PinnedItem->GetListViewPtr(); ListView.IsValid())
	{
		const TSharedPtr<SConsoleVariablesEditorList> PinnedListView = ListView.Pin();
		PinnedListView->OnListItemCheckBoxStateChange(InNewState);
	}
}

TSharedRef<SWidget> SConsoleVariablesEditorListRow::GenerateValueCellWidget(
	const TSharedPtr<FConsoleVariablesEditorListRow> PinnedItem)
{
	if (PinnedItem->GetCommandInfo().IsValid())
	{
		ValueChildInputWidget = SConsoleVariablesEditorListValueInput::GetInputWidget(Item);
		const TSharedRef<SHorizontalBox> FinalValueWidget =
			SNew(SHorizontalBox)
			.ToolTipText_Lambda([this, PinnedItem]()
			{
				if (const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedCommand = PinnedItem->GetCommandInfo().
						Pin();
					PinnedCommand.IsValid())
				{
					return FText::Format(
						ValueWidgetToolTipFormatText,
						FText::FromString(PinnedItem->GetCachedValue()),
						FText::FromString(PinnedItem->GetPresetValue()),
						FText::FromString(PinnedCommand->StartupValueAsString),
						FConsoleVariablesEditorCommandInfo::ConvertConsoleVariableSetByFlagToText(
							PinnedCommand->StartupSource)
					);
				}

				return FText::GetEmpty();
			});

		FinalValueWidget->AddSlot()
		                .VAlign(VAlign_Center)
		                .Padding(FMargin(2, 0))
		[
			ValueChildInputWidget.ToSharedRef()
		];

		FinalValueWidget->AddSlot()
		                .AutoWidth()
		                .VAlign(VAlign_Center)
		                .Padding(FMargin(2, 0))
		[
			SNew(SButton)
			.IsFocusable(false)
			.ToolTipText(FText::Format(RevertButtonFormatText, FText::FromString(PinnedItem->GetPresetValue())))
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("NoBorder"))
			.ContentPadding(0)
			.Visibility_Lambda([this, PinnedItem]()
            {
	            if (PinnedItem.IsValid() && PinnedItem->GetRowType() ==
		            FConsoleVariablesEditorListRow::SingleCommand)
	            {
		            const TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo =
			            PinnedItem->GetCommandInfo().Pin();
		            
					if (CommandInfo.IsValid() &&
						CommandInfo->ObjectType == FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable)
		            {
			            PinnedItem->SetDoesCurrentValueDifferFromPresetValue(
							 CommandInfo->IsCurrentValueDifferentFromInputValue(
								 PinnedItem->GetPresetValue()));

						 return PinnedItem->IsRowChecked() && PinnedItem->
								DoesCurrentValueDifferFromPresetValue()
									? EVisibility::Visible
									: EVisibility::Collapsed;
		            }
	            }

	            return EVisibility::Collapsed;
            })
			.OnClicked_Lambda([this, PinnedItem]()
            {
				for (const FConsoleVariablesEditorListRowPtr& RowPtr : PinnedItem->GetRowsAffectedByActions())
				{
					const TSharedPtr<FConsoleVariablesEditorCommandInfo> CommandInfo =
						PinnedItem->GetCommandInfo().Pin();

					if (!CommandInfo.IsValid() ||
						CommandInfo->ObjectType != FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable)
					{
						continue;
					}

					RowPtr->ResetToPresetValue();
				}

	            return FReply::Handled();
            })
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("PropertyWindow.DiffersFromDefault"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];

		return FinalValueWidget;
	}

	return SNullWidget::NullWidget;
}

void SConsoleVariablesEditorListRowHoverWidgets::Construct(const FArguments& InArgs,
                                                           const TWeakPtr<FConsoleVariablesEditorListRow> InRow)
{
	check(InRow.IsValid());

	Item = InRow;

	const FText ButtonTooltip = LOCTEXT("RemoveCvarTooltip", "Remove cvar from this list and reset its value to the startup value.");
	const FSlateBrush* ButtonImage = FAppStyle::Get().GetBrush("Icons.Delete");

	ChildSlot
	[
		// Action Button
		SNew(SButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("NoBorder"))
		.Visibility(EVisibility::Visible)
		.ToolTipText(this, &SConsoleVariablesEditorListRowHoverWidgets::GetButtonTooltip)
		.ButtonColorAndOpacity(FStyleColors::Transparent)
		.ContentPadding(0.f)
		.OnClicked_Lambda([this]()
		{
			return Item.Pin()->OnActionButtonClicked();
		})
		[
			SNew(SScaleBox)
			[
				SNew(SImage)
				.Visibility(EVisibility::SelfHitTestInvisible)
				.Image(this, &SConsoleVariablesEditorListRowHoverWidgets::GetButtonImage)
				.ColorAndOpacity(this, &SConsoleVariablesEditorListRowHoverWidgets::GetActionButtonColorAndOpacity)
			]
		]
	];
}

SConsoleVariablesEditorListRowHoverWidgets::~SConsoleVariablesEditorListRowHoverWidgets()
{
	Item.Reset();
}

FSlateColor SConsoleVariablesEditorListRowHoverWidgets::GetActionButtonColorAndOpacity() const
{
	if (IsHovered())
	{
		return FSlateColor::UseForeground();
	}

	return FSlateColor::UseSubduedForeground();
}

const FSlateBrush* SConsoleVariablesEditorListRowHoverWidgets::GetButtonImage() const
{
	const FConsoleVariablesEditorListRowPtr RowPtr = Item.Pin();
	if (RowPtr.IsValid() && RowPtr->IsGlobalSearch())
	{
		return RowPtr->IsInPreset()
			? FAppStyle::Get().GetBrush("Icons.Star")
			: FConsoleVariablesEditorStyle::Get().GetBrush("ConsoleVariables.Favorite.Outline.Small");
	}

	return FAppStyle::Get().GetBrush("Icons.Delete");
}

FText SConsoleVariablesEditorListRowHoverWidgets::GetButtonTooltip() const
{
	const FConsoleVariablesEditorListRowPtr RowPtr = Item.Pin();
	if (RowPtr.IsValid() && RowPtr->IsGlobalSearch())
	{
		return RowPtr->IsInPreset()
			? LOCTEXT("RemoveCvarFromPresetTooltip", "Remove this cvar from your current preset.")
			: LOCTEXT("AddCvarToPresetTooltip", "Add this cvar to your current preset.");
	}

	return LOCTEXT("RemoveCvarTooltip", "Remove cvar from this list and reset its value to the startup value.");
}

#undef LOCTEXT_NAMESPACE
