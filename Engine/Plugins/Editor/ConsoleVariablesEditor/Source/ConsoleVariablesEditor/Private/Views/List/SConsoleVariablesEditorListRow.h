// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConsoleVariablesEditorListRow.h"
#include "SConsoleVariablesEditorList.h"

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/IToolTip.h"

class SConsoleVariablesEditorListValueInput;
class SConsoleVariablesEditorListRowHoverWidgets;

class SConsoleVariablesEditorListRow : public SMultiColumnTableRow<FConsoleVariablesEditorListRowPtr>
{
public:
	
	SLATE_BEGIN_ARGS(SConsoleVariablesEditorListRow)
	{}

	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const TWeakPtr<FConsoleVariablesEditorListRow> InRow);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual ~SConsoleVariablesEditorListRow() override;

	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	void HandleDragLeave(const FDragDropEvent& DragDropEvent);
	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FConsoleVariablesEditorListRowPtr TargetItem);
	FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FConsoleVariablesEditorListRowPtr TargetItem);
	
	void FlashRow();

	EVisibility GetHoverableWidgetsVisibility() const;
	EVisibility GetFlashImageVisibility() const;
	FSlateColor GetFlashImageColorAndOpacity() const;

	static const FSlateBrush* GetBorderImage(const FConsoleVariablesEditorListRow::EConsoleVariablesEditorListRowType InRowType);

	TSharedRef<SWidget> GenerateCells(const FName& InColumnName, const TSharedPtr<FConsoleVariablesEditorListRow> PinnedItem);

	ECheckBoxState GetCheckboxState() const;
	void OnCheckboxStateChange(const ECheckBoxState InNewState) const;

	TSharedRef<SWidget> GenerateValueCellWidget(const TSharedPtr<FConsoleVariablesEditorListRow> PinnedItem);

	#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

	const FText ValueWidgetToolTipFormatText = LOCTEXT("ValueWidgetToolTipFormatText", "Custom Value: {0}\nPreset Value: {1}\nStartup Value: {2} (Set By {3})");
	const FText RevertButtonFormatText = LOCTEXT("RevertButtonFormatText", "Reset to Preset Value: {0}");

	const FText InsertFormatText = LOCTEXT("InsertAboveFormatText", "Insert {0} {1} {2}");
	const FText AboveText = LOCTEXT("AboveListItem", "above");
	const FText BelowText = LOCTEXT("BelowListItem", "below");
	static const inline FText MultiDragFormatText = LOCTEXT("MultiDragFormatText", "{0} Items");

	#undef LOCTEXT_NAMESPACE

private:
	
	TWeakPtr<FConsoleVariablesEditorListRow> Item;
	
	TSharedPtr<IToolTip> HoverToolTip;

	TArray<TSharedPtr<SImage>> FlashImages;

	TSharedPtr<SConsoleVariablesEditorListValueInput> ValueChildInputWidget;
	
	TSharedPtr<SConsoleVariablesEditorListRowHoverWidgets> HoverableWidgetsPtr;

	FCurveSequence FlashAnimation;

	const float FlashAnimationDuration = 0.75f;
	const FLinearColor FlashColor = FLinearColor::White;

	/** The offset applied to text widgets so that the text aligns with the column header text */
	float TextBlockLeftPadding = 5.0f;
};

class SConsoleVariablesEditorListRowHoverWidgets : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SConsoleVariablesEditorListRowHoverWidgets)
	{}

	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TWeakPtr<FConsoleVariablesEditorListRow> InRow);

	virtual ~SConsoleVariablesEditorListRowHoverWidgets() override;

private:

	FSlateColor GetActionButtonColorAndOpacity() const;
	const FSlateBrush* GetButtonImage() const;
	FText GetButtonTooltip() const;
	
	TWeakPtr<FConsoleVariablesEditorListRow> Item;
};
