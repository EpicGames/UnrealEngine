// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConsoleVariablesEditorListValueInput.h"

#include "ConsoleVariablesEditorListRow.h"
#include "ConsoleVariablesEditorModule.h"
#include "ConsoleVariablesEditorProjectSettings.h"

#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

SConsoleVariablesEditorListValueInput::~SConsoleVariablesEditorListValueInput()
{
	Item.Reset();
}

TSharedRef<SConsoleVariablesEditorListValueInput> SConsoleVariablesEditorListValueInput::GetInputWidget(
	const TWeakPtr<FConsoleVariablesEditorListRow> InRow)
{
	const FConsoleVariablesEditorListRowPtr PinnedItem = InRow.Pin();

	check(PinnedItem);
	
	if (PinnedItem->GetCommandInfo().IsValid())
	{
		const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedInfo = PinnedItem->GetCommandInfo().Pin();

		if (const IConsoleVariable* Variable = PinnedInfo->GetConsoleVariablePtr())
		{
			if (Variable->IsVariableFloat())
			{
				return SNew(SConsoleVariablesEditorListValueInput_Float, InRow);
			}

			if (Variable->IsVariableBool())
			{
				return SNew(SConsoleVariablesEditorListValueInput_Bool, InRow);
			}

			if (Variable->IsVariableInt())
			{
				return SNew(SConsoleVariablesEditorListValueInput_Int, InRow);
			}

			if (Variable->IsVariableString())
			{
				return SNew(SConsoleVariablesEditorListValueInput_String, InRow);
			}

			// Showflags are not considered to be any of these types, but they should be ints with a min/max of 0/2
			if (PinnedInfo->Command.Contains("showflag", ESearchCase::IgnoreCase))
			{
				return SNew(SConsoleVariablesEditorListValueInput_Int, InRow, true);
			}
		}

		// For Commands
		const FString CachedValue = PinnedItem->GetCachedValue();
		return SNew(SConsoleVariablesEditorListValueInput_Command, InRow,
			CachedValue.IsEmpty() ? PinnedItem->GetPresetValue() : CachedValue);
	}

	// fallback
	return SNew(SConsoleVariablesEditorListValueInput_String, InRow);
}

bool SConsoleVariablesEditorListValueInput::IsRowChecked() const
{
	return Item.Pin()->IsRowChecked();
}

void SConsoleVariablesEditorListValueInput_Float::Construct(const FArguments& InArgs,
                                                            const TWeakPtr<FConsoleVariablesEditorListRow> InRow)
{
	check (InRow.IsValid());
	
	Item = InRow;

	ProjectSettingsPtr = GetMutableDefault<UConsoleVariablesEditorProjectSettings>();
	
	ChildSlot
	[
		SAssignNew(InputWidget, SSpinBox<float>)
		.MaxFractionalDigits(3)
		.Value_Lambda([this]
		{
			check (Item.IsValid());
			
			if (Item.Pin()->GetWidgetCheckedState() == ECheckBoxState::Checked ||
					(ProjectSettingsPtr &&
						ProjectSettingsPtr->UncheckedRowDisplayType == EConsoleVariablesEditorRowDisplayType::ShowCurrentValue))
			{
				if (const IConsoleVariable* AsVariable = Item.Pin()->GetCommandInfo().Pin()->GetConsoleVariablePtr())
				{
					return FCString::Atof(*FString::SanitizeFloat(AsVariable->GetFloat()));
				}
			}

			return FCString::Atof(*Item.Pin()->GetCachedValue());
		})
		.OnValueChanged(this, &SConsoleVariablesEditorListValueInput_Float::OnSliderAffected, false)
		.OnValueCommitted_Lambda([this] (const float InValue, ETextCommit::Type CommitType)
		{
			if (CommitType != ETextCommit::Default)
			{
				OnSliderAffected(InValue, true);
			}
		})
		.IsEnabled(this, &SConsoleVariablesEditorListValueInput::IsRowChecked)
	];

	Item.Pin()->SetCachedValue(GetInputValueAsString());
}

SConsoleVariablesEditorListValueInput_Float::~SConsoleVariablesEditorListValueInput_Float()
{
	InputWidget.Reset();
}

void SConsoleVariablesEditorListValueInput_Float::SetInputValue(const FString& InValueAsString)
{
	InputWidget->SetValue(FCString::Atof(*InValueAsString));
}

FString SConsoleVariablesEditorListValueInput_Float::GetInputValueAsString()
{
	return FString::SanitizeFloat(GetInputValue());
}

float SConsoleVariablesEditorListValueInput_Float::GetInputValue() const
{
	return InputWidget->GetValue();
}

void SConsoleVariablesEditorListValueInput_Float::OnSliderAffected(const float InValue, const bool bPrintCommand)
{
	if (!Item.IsValid())
	{
		return;
	}
	
	const TSharedPtr<FConsoleVariablesEditorListRow> PinnedItem = Item.Pin();
					
	const FString ValueAsString = FString::SanitizeFloat(InValue);

	for (const FConsoleVariablesEditorListRowPtr& RowPtr : PinnedItem->GetRowsAffectedByActions())
	{
		if (RowPtr->GetCommandInfo().Pin()->ObjectType != FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable)
		{
			continue;
		}

		RowPtr->GetCommandInfo().Pin()->ExecuteCommand(ValueAsString, true, true, !bPrintCommand);
		RowPtr->SetCachedValue(ValueAsString);
	}
}

void SConsoleVariablesEditorListValueInput_Int::Construct(const FArguments& InArgs,
                                                          const TWeakPtr<FConsoleVariablesEditorListRow> InRow,
                                                          const bool bIsShowFlag)
{
	check (InRow.IsValid());
	
	Item = InRow;

	ProjectSettingsPtr = GetMutableDefault<UConsoleVariablesEditorProjectSettings>();
	
	ChildSlot
	[
		SAssignNew(InputWidget, SSpinBox<int32>)
		.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("NumericEntrySpinBox"))
		.Value_Lambda([this]
		{
			check (Item.IsValid());
			
			if (Item.Pin()->GetWidgetCheckedState() == ECheckBoxState::Checked ||
					(ProjectSettingsPtr &&
						ProjectSettingsPtr->UncheckedRowDisplayType == EConsoleVariablesEditorRowDisplayType::ShowCurrentValue))
			{
				if (const IConsoleVariable* AsVariable = Item.Pin()->GetCommandInfo().Pin()->GetConsoleVariablePtr())
				{
					return FCString::Atoi(*AsVariable->GetString());
				}
			}

			return FCString::Atoi(*Item.Pin()->GetCachedValue());
		})
		.OnValueChanged(this, &SConsoleVariablesEditorListValueInput_Int::OnSliderAffected, false)
		.OnValueCommitted_Lambda([this] (const int32 InValue, ETextCommit::Type CommitType)
		{
			if (CommitType != ETextCommit::Default)
			{
				OnSliderAffected(InValue, true);
			}
		})
		.IsEnabled(this, &SConsoleVariablesEditorListValueInput::IsRowChecked)
	];

	if (bIsShowFlag)
	{
		InputWidget->SetMinSliderValue(0);
		InputWidget->SetMaxSliderValue(2);

		const int32 PresetValue = FCString::Atoi(*Item.Pin()->GetPresetValue());

		Item.Pin()->SetPresetValue(FString::FromInt(FMath::Clamp(PresetValue, 0, 2)));
	}
	
	Item.Pin()->SetCachedValue(GetInputValueAsString());
}

SConsoleVariablesEditorListValueInput_Int::~SConsoleVariablesEditorListValueInput_Int()
{
	InputWidget.Reset();
}

void SConsoleVariablesEditorListValueInput_Int::SetInputValue(const FString& InValueAsString)
{
	if (InValueAsString.IsNumeric())
	{
		InputWidget->SetValue(FCString::Atoi(*InValueAsString));
	}
	else
	{
		InputWidget->SetValue(2);
		
		if (InValueAsString.TrimStartAndEnd().ToLower() == "true")
		{
			InputWidget->SetValue(1);
		}
		else if (InValueAsString.TrimStartAndEnd().ToLower() == "false")
		{
			InputWidget->SetValue(0);
		}
	}
}

FString SConsoleVariablesEditorListValueInput_Int::GetInputValueAsString()
{
	return FString::FromInt(GetInputValue());
}

int32 SConsoleVariablesEditorListValueInput_Int::GetInputValue() const
{
	return InputWidget->GetValue();
}

void SConsoleVariablesEditorListValueInput_Int::OnSliderAffected(const int32 InValue, const bool bPrintCommand)
{
	if (!Item.IsValid())
	{
		return;
	}

	const TSharedPtr<FConsoleVariablesEditorListRow> PinnedItem = Item.Pin();
					
	const FString ValueAsString = FString::FromInt(InValue);

	for (const FConsoleVariablesEditorListRowPtr& RowPtr : PinnedItem->GetRowsAffectedByActions())
	{
		if (RowPtr->GetCommandInfo().Pin()->ObjectType != FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable)
		{
			continue;
		}

		if (!RowPtr->GetCachedValue().Equals(ValueAsString))
		{
			RowPtr->GetCommandInfo().Pin()->ExecuteCommand(ValueAsString, true, true, !bPrintCommand);
			RowPtr->SetCachedValue(ValueAsString);

			continue;
		}

		// If the new value and cached value aren't different the command won't be executed.
		// If we still want to print the command, do it here
		if (bPrintCommand)
		{
			RowPtr->GetCommandInfo().Pin()->PrintCommandOrVariable();
		}
	}
}

void SConsoleVariablesEditorListValueInput_String::Construct(const FArguments& InArgs,
                                                             const TWeakPtr<FConsoleVariablesEditorListRow> InRow)
{
	check (InRow.IsValid());
	
	Item = InRow;

	ProjectSettingsPtr = GetMutableDefault<UConsoleVariablesEditorProjectSettings>();
	
	ChildSlot
	[
		SAssignNew(InputWidget, SEditableTextBox)
		.Text_Lambda([this]
		{
			check (Item.IsValid());
			
			if (Item.Pin()->GetWidgetCheckedState() == ECheckBoxState::Checked ||
					(ProjectSettingsPtr &&
						ProjectSettingsPtr->UncheckedRowDisplayType == EConsoleVariablesEditorRowDisplayType::ShowCurrentValue))
			{
				if (const IConsoleVariable* AsVariable = Item.Pin()->GetCommandInfo().Pin()->GetConsoleVariablePtr())
				{
					return FText::FromString(*AsVariable->GetString());
				}
			}

			return FText::FromString(Item.Pin()->GetCachedValue());
		})
		.OnTextCommitted_Lambda([this] (const FText& InValue, ETextCommit::Type InTextCommitType)
		{
			check (Item.IsValid());

			const TSharedPtr<FConsoleVariablesEditorListRow> PinnedItem = Item.Pin();

			const FString ValueAsString = InValue.ToString();

			for (const FConsoleVariablesEditorListRowPtr& RowPtr : PinnedItem->GetRowsAffectedByActions())
			{
				if (RowPtr->GetCommandInfo().Pin()->ObjectType != FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable)
				{
					continue;
				}

				if (!RowPtr->GetCachedValue().Equals(ValueAsString))
				{
					RowPtr->GetCommandInfo().Pin()->ExecuteCommand(ValueAsString);
					RowPtr->SetCachedValue(ValueAsString);
				}
			}
		})
		.IsEnabled(this, &SConsoleVariablesEditorListValueInput::IsRowChecked)
	];

	Item.Pin()->SetCachedValue(GetInputValueAsString());
}

SConsoleVariablesEditorListValueInput_String::~SConsoleVariablesEditorListValueInput_String()
{
	InputWidget.Reset();
}

void SConsoleVariablesEditorListValueInput_String::SetInputValue(const FString& InValueAsString)
{
	InputWidget->SetText(FText::FromString(InValueAsString));
}

FString SConsoleVariablesEditorListValueInput_String::GetInputValueAsString()
{
	return GetInputValue();
}

FString SConsoleVariablesEditorListValueInput_String::GetInputValue() const
{
	return InputWidget->GetText().ToString();
}

void SConsoleVariablesEditorListValueInput_Bool::Construct(const FArguments& InArgs,
                                                           const TWeakPtr<FConsoleVariablesEditorListRow> InRow)
{
	check (InRow.IsValid());
	
	Item = InRow;

	ProjectSettingsPtr = GetMutableDefault<UConsoleVariablesEditorProjectSettings>();
	
	ChildSlot
	[
		SAssignNew(InputWidget, SButton)
		.OnClicked_Lambda([this] ()
		{
			const bool bValueAsBool = GetInputValue();

			SetInputValue(!bValueAsBool);

			return FReply::Handled();
		})
		.IsEnabled(this, &SConsoleVariablesEditorListValueInput::IsRowChecked)
		[
			SAssignNew(ButtonText, STextBlock)
			.Justification(ETextJustify::Center)
			.Text_Lambda([this]()
			{
				check (Item.IsValid());
			
				if (Item.Pin()->GetWidgetCheckedState() == ECheckBoxState::Checked ||
						(ProjectSettingsPtr &&
							ProjectSettingsPtr->UncheckedRowDisplayType == EConsoleVariablesEditorRowDisplayType::ShowCurrentValue))
				{
					if (const IConsoleVariable* AsVariable = Item.Pin()->GetCommandInfo().Pin()->GetConsoleVariablePtr())
					{
						return FText::FromString(AsVariable->GetString());
					}
				}

				return FText::FromString(Item.Pin()->GetCachedValue());
			})
		]
	];
	
	Item.Pin()->SetCachedValue(GetInputValueAsString());
}

SConsoleVariablesEditorListValueInput_Bool::~SConsoleVariablesEditorListValueInput_Bool()
{
	InputWidget.Reset();
	ButtonText.Reset();
}

void SConsoleVariablesEditorListValueInput_Bool::SetInputValue(const FString& InValueAsString)
{
	if (!Item.IsValid())
	{
		return;
	}

	const TSharedPtr<FConsoleVariablesEditorListRow> PinnedItem = Item.Pin();
	
	for (const FConsoleVariablesEditorListRowPtr& RowPtr : PinnedItem->GetRowsAffectedByActions())
	{
		if (RowPtr->GetCommandInfo().Pin()->ObjectType != FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Variable)
		{
			continue;
		}

		if (!RowPtr->GetCachedValue().Equals(InValueAsString))
		{
			RowPtr->GetCommandInfo().Pin()->ExecuteCommand(InValueAsString);
			RowPtr->SetCachedValue(InValueAsString);
		}
	}
}

void SConsoleVariablesEditorListValueInput_Bool::SetInputValue(const bool bNewValue)
{	
	SetInputValue(BoolToString(bNewValue));
}

FString SConsoleVariablesEditorListValueInput_Bool::GetInputValueAsString()
{
	return ButtonText->GetText().ToString();
}

bool SConsoleVariablesEditorListValueInput_Bool::GetInputValue()
{
	return StringToBool(GetInputValueAsString());
}

void SConsoleVariablesEditorListValueInput_Command::Construct(const FArguments& InArgs,
	const TWeakPtr<FConsoleVariablesEditorListRow> InRow, const FString& InSavedText)
{
	check (InRow.IsValid());
	
	Item = InRow;

	ProjectSettingsPtr = GetMutableDefault<UConsoleVariablesEditorProjectSettings>();
	
	ChildSlot
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(InputText, SEditableTextBox)
			.Text(FText::FromString(InSavedText))
			.HintText(LOCTEXT("CommandValueTypeRowInputHintText", "Value..."))
			.IsEnabled(this, &SConsoleVariablesEditorListValueInput::IsRowChecked)
			.Visibility(Item.Pin()->GetCommandInfo().Pin()->ObjectType ==
				FConsoleVariablesEditorCommandInfo::EConsoleObjectType::Command ?
				EVisibility::Visible : EVisibility::Collapsed)
		]

		+SHorizontalBox::Slot()
		.Padding(FMargin(2.f, 0, 0, 0))
		.VAlign(VAlign_Fill)
		[
			SAssignNew(InputWidget, SButton)
			.OnClicked_Lambda([this] ()
			{
				check (Item.IsValid());
				
				if (const TSharedPtr<FConsoleVariablesEditorListRow> PinnedItem = Item.Pin())
				{					
					if (const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedCommand =
						PinnedItem->GetCommandInfo().Pin())
					{
						const FString InputValueAsString = InputText->GetText().ToString();
						
						PinnedCommand->ExecuteCommand(InputValueAsString);

						PinnedItem->SetCachedValue(InputValueAsString);
					}

					return FReply::Handled();
				}

				return FReply::Unhandled();
			})
			.IsEnabled(this, &SConsoleVariablesEditorListValueInput::IsRowChecked)
			.ContentPadding(FMargin(0.f))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.Padding(FMargin(2.f, 0, 0, 0))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(LOCTEXT("ConsoleCommandExecutionButtonText", "Execute"))
				]
			]
		]
	];

	Item.Pin()->SetCachedValue(GetInputValueAsString());
}

SConsoleVariablesEditorListValueInput_Command::~SConsoleVariablesEditorListValueInput_Command()
{
	InputWidget.Reset();
	InputText.Reset();
}

void SConsoleVariablesEditorListValueInput_Command::SetInputValue(const FString& InValueAsString)
{
	InputText->SetText(FText::FromString(InValueAsString));
}

FString SConsoleVariablesEditorListValueInput_Command::GetInputValueAsString()
{
	return GetInputValue();
}

FString SConsoleVariablesEditorListValueInput_Command::GetInputValue() const
{
	return InputText->GetText().ToString();
}

#undef LOCTEXT_NAMESPACE
