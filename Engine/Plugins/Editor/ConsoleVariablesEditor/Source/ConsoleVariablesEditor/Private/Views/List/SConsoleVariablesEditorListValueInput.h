// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

struct FConsoleVariablesEditorListRow;

class UConsoleVariablesEditorProjectSettings;

class SConsoleVariablesEditorListValueInput : public SCompoundWidget
{
public:
	virtual ~SConsoleVariablesEditorListValueInput() override;

	static TSharedRef<SConsoleVariablesEditorListValueInput> GetInputWidget(const TWeakPtr<FConsoleVariablesEditorListRow> InRow);
	
	virtual void SetInputValue(const FString& InValueAsString) = 0;
	virtual FString GetInputValueAsString() = 0;

	bool IsRowChecked() const;

protected:
	
	TWeakPtr<FConsoleVariablesEditorListRow> Item = nullptr;

	TObjectPtr<UConsoleVariablesEditorProjectSettings> ProjectSettingsPtr = nullptr;
};

class SConsoleVariablesEditorListValueInput_Float : public SConsoleVariablesEditorListValueInput
{
public:
	
	SLATE_BEGIN_ARGS(SConsoleVariablesEditorListValueInput_Float)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FConsoleVariablesEditorListRow> InRow);
	
	virtual ~SConsoleVariablesEditorListValueInput_Float() override;
	
	virtual void SetInputValue(const FString& InValueAsString) override;
	virtual FString GetInputValueAsString() override;

	float GetInputValue() const;
	void OnSliderAffected(const float InValue, const bool bPrintCommand);

private:
	
	TSharedPtr<SSpinBox<float>> InputWidget;
};

class SConsoleVariablesEditorListValueInput_Int : public SConsoleVariablesEditorListValueInput
{
public:
	
	SLATE_BEGIN_ARGS(SConsoleVariablesEditorListValueInput_Int)
	{}

	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs, const TWeakPtr<FConsoleVariablesEditorListRow> InRow, const bool bIsShowFlag = false);

	virtual ~SConsoleVariablesEditorListValueInput_Int() override;
	
	virtual void SetInputValue(const FString& InValueAsString) override;
	virtual FString GetInputValueAsString() override;

	int32 GetInputValue() const;
	void OnSliderAffected(const int32 InValue, const bool bPrintCommand);

private:
	
	TSharedPtr<SSpinBox<int32>> InputWidget;
};

class SConsoleVariablesEditorListValueInput_String : public SConsoleVariablesEditorListValueInput
{
public:
	
	SLATE_BEGIN_ARGS(SConsoleVariablesEditorListValueInput_String)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FConsoleVariablesEditorListRow> InRow);

	virtual ~SConsoleVariablesEditorListValueInput_String() override;
	
	virtual void SetInputValue(const FString& InValueAsString) override;
	virtual FString GetInputValueAsString() override;
	
	FString GetInputValue() const;

private:
	
	TSharedPtr<SEditableTextBox> InputWidget;
};

class SConsoleVariablesEditorListValueInput_Bool : public SConsoleVariablesEditorListValueInput
{
public:
	
	SLATE_BEGIN_ARGS(SConsoleVariablesEditorListValueInput_Bool)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FConsoleVariablesEditorListRow> InRow);

	virtual ~SConsoleVariablesEditorListValueInput_Bool() override;
	
	virtual void SetInputValue(const FString& InValueAsString) override;
	void SetInputValue(const bool bNewValue);
	virtual FString GetInputValueAsString() override;

	bool GetInputValue();

	static bool StringToBool(const FString& InString)
	{
		return InString.TrimStartAndEnd().ToLower() == "true";
	}

	static FString BoolToString(const bool bNewBool)
	{
		return bNewBool ? "true" : "false";
	}

private:
	
	TSharedPtr<SButton> InputWidget;
	TSharedPtr<STextBlock> ButtonText;
};

class SConsoleVariablesEditorListValueInput_Command : public SConsoleVariablesEditorListValueInput
{
public:
	
	SLATE_BEGIN_ARGS(SConsoleVariablesEditorListValueInput_Command)
	{}

	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs, const TWeakPtr<FConsoleVariablesEditorListRow> InRow, const FString& InSavedText);
	
	virtual ~SConsoleVariablesEditorListValueInput_Command() override;
	
	virtual void SetInputValue(const FString& InValueAsString) override;
	virtual FString GetInputValueAsString() override;

	FString GetInputValue() const;

private:
	
	TSharedPtr<SButton> InputWidget;
	TSharedPtr<SEditableTextBox> InputText;
};
