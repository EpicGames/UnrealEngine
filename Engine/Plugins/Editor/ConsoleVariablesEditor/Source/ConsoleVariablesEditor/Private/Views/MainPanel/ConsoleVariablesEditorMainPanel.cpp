// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/MainPanel/ConsoleVariablesEditorMainPanel.h"

#include "ConcertMessages.h"
#include "ConsoleVariablesAsset.h"
#include "ConsoleVariablesEditorCommandInfo.h"
#include "ConsoleVariablesEditorLog.h"
#include "ConsoleVariablesEditorModule.h"
#include "ConsoleVariablesEditorProjectSettings.h"
#include "HAL/IConsoleManager.h"
#include "MultiUser/ConsoleVariableSync.h"
#include "Views/List/ConsoleVariablesEditorList.h"
#include "Views/MainPanel/SConsoleVariablesEditorMainPanel.h"

#include "Editor/EditorEngine.h"
#include "FileHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "MultiUser/ConsoleVariableSyncData.h"
#include "UnrealEngine.h"

FConsoleVariablesEditorMainPanel::FConsoleVariablesEditorMainPanel()
{
	EditorList = MakeShared<FConsoleVariablesEditorList>();
	
	OnConnectionChangedHandle = MultiUserManager.OnConnectionChange().AddStatic(
		&FConsoleVariablesEditorMainPanel::OnConnectionChanged);
	OnRemoteCVarChangeHandle = MultiUserManager.OnRemoteCVarChange().AddStatic(
		&FConsoleVariablesEditorMainPanel::OnRemoteCvarChange);
	OnRemoteListItemCheckStateChangeHandle = MultiUserManager.OnRemoteListItemCheckStateChange().AddStatic(
			&FConsoleVariablesEditorMainPanel::OnRemoteListItemCheckStateChange);
}

FConsoleVariablesEditorMainPanel::~FConsoleVariablesEditorMainPanel()
{
	MainPanelWidget.Reset();
	EditorList.Reset();
	MultiUserManager.OnConnectionChange().Remove(OnConnectionChangedHandle);
	MultiUserManager.OnRemoteCVarChange().Remove(OnRemoteCVarChangeHandle);
	MultiUserManager.OnRemoteListItemCheckStateChange().Remove(OnRemoteListItemCheckStateChangeHandle);
}

TSharedRef<SWidget> FConsoleVariablesEditorMainPanel::GetOrCreateWidget()
{
	if (!MainPanelWidget.IsValid())
	{
		SAssignNew(MainPanelWidget, SConsoleVariablesEditorMainPanel, SharedThis(this));
	}

	return MainPanelWidget.ToSharedRef();
}

FConsoleVariablesEditorModule& FConsoleVariablesEditorMainPanel::GetConsoleVariablesModule()
{
	return FConsoleVariablesEditorModule::Get();
}

TObjectPtr<UConsoleVariablesAsset> FConsoleVariablesEditorMainPanel::GetEditingAsset()
{
	return GetConsoleVariablesModule().GetPresetAsset();
}

void FConsoleVariablesEditorMainPanel::AddConsoleObjectToCurrentPreset(
	const FString InConsoleCommand, const FString InValue, const bool bScrollToNewRow) const
{
	if (const TObjectPtr<UConsoleVariablesAsset> Asset = GetEditingAsset())
	{
		const FString TrimmedCommand = InConsoleCommand.TrimStartAndEnd();
		FConsoleVariablesEditorAssetSaveData ExistingData;
		const bool bHadCommandInAsset = Asset->FindSavedDataByCommandString(TrimmedCommand, ExistingData, ESearchCase::IgnoreCase);

		Asset->AddOrSetConsoleObjectSavedData(
			{
				TrimmedCommand,
				InValue,
				ECheckBoxState::Checked
			}
		);

		if (GetEditorList().Pin()->GetListMode() == FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset)
		{
			RebuildList(bScrollToNewRow ? InConsoleCommand : "");
		}
	}
}

void FConsoleVariablesEditorMainPanel::UpdateCachedValue(const FString& InConsoleVariable, const FString& InValue)
{
	if (TSharedPtr<FConsoleVariablesEditorList> TheEditorList = GetEditorList().Pin())
	{
		TheEditorList->UpdateCachedValue(InConsoleVariable, InValue);
	}
}

void FConsoleVariablesEditorMainPanel::OnRemoteCvarChanged()
{
	if (GetEditorListMode() == FConsoleVariablesEditorList::EConsoleVariablesEditorListMode::Preset)
	{
		RebuildList();
	}
}

FReply FConsoleVariablesEditorMainPanel::ValidateConsoleInputAndAddToCurrentPreset(const FText& CommittedText) const
{
	// Clear Search
	if (const TSharedPtr<FConsoleVariablesEditorList> PinnedList = GetEditorList().Pin())
	{
		PinnedList->SetSearchString("");
	}
	
	const FString CommandString = CommittedText.ToString().TrimStartAndEnd();

	if (CommandString.IsEmpty())
	{
		UE_LOGF(LogConsoleVariablesEditor, Warning, "%s: Input is blank.", __FUNCTION__);

		return FReply::Unhandled();
	}
	
	FString CommandKey; // The actual variable name, like r.ScreenPercentage
	FString ValueString; // The variable value, usually after the CommandKey and a single space
	FString AdditionalParams; // Some variables have multiple parameters between the CommandKey and ValueString

	if (CommandString.Contains(" "))
	{
		// We want to treat the last string after the last space as the value
		// This allows commands with multiple parameters to be treated as separate list items
		const int32 IndexOfLastSpace = CommandString.Find(" ", ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		CommandKey = CommandString.Left(IndexOfLastSpace).TrimStartAndEnd();
		ValueString = CommandString.RightChop(IndexOfLastSpace).TrimStartAndEnd();

		// If true, this variable has additional parameters
		if (CommandKey.Contains(" "))
		{
			const int32 IndexOfFirstSpace = CommandString.Find(" ", ESearchCase::IgnoreCase, ESearchDir::FromStart);
			AdditionalParams = CommandKey.RightChop(IndexOfFirstSpace).TrimStartAndEnd();
			CommandKey = CommandKey.Left(IndexOfFirstSpace).TrimStartAndEnd();
		}
	}
	else
	{
		CommandKey = CommandString;
	}

	if (IConsoleObject* ConsoleObject = IConsoleManager::Get().FindConsoleObject(*CommandKey)) 
	{
		if (ValueString.IsEmpty())
		{
			if (const IConsoleVariable* AsVariable = ConsoleObject->AsVariable())
			{
				ValueString = AsVariable->GetString();
			}
		}
		
		AddConsoleObjectToCurrentPreset(
			CommandKey + " " + AdditionalParams,
			ValueString,
			true
		);
	}
	// Try to execute the whole command. Some commands are not registered, but are parsed externally so they won't be found by IConsoleManager::FindConsoleObject
	else if (GEngine->Exec(FConsoleVariablesEditorCommandInfo::GetCurrentWorld(), *CommandString))
	{
		AddConsoleObjectToCurrentPreset(
			CommandString,
			"",
			true
			);
	}
	else
	{
		UE_LOGF(LogConsoleVariablesEditor, Warning,
			"%s: Input '%ls' is not a recognized console variable or command.",
			__FUNCTION__, *CommandString);

		return FReply::Unhandled();
	}

	return FReply::Handled();
}

void FConsoleVariablesEditorMainPanel::RebuildList(const FString InConsoleCommandToScrollTo, bool bShouldCacheValues) const
{
	if (EditorList.IsValid())
	{
		EditorList->RebuildList(InConsoleCommandToScrollTo);
	}
}

void FConsoleVariablesEditorMainPanel::RefreshList() const
{
	if (EditorList.IsValid())
	{
		EditorList->RefreshList();
	}
}

void FConsoleVariablesEditorMainPanel::UpdatePresetValuesForSave(TObjectPtr<UConsoleVariablesAsset> InAsset) const
{
	if (EditorList.IsValid())
	{
		EditorList->UpdatePresetValuesForSave(InAsset);
	}
}

void FConsoleVariablesEditorMainPanel::RefreshMultiUserDetails() const
{
	if (MainPanelWidget.IsValid())
	{
		MainPanelWidget->RefreshMultiUserDetails();
	}
}

void FConsoleVariablesEditorMainPanel::SaveCurrentPreset()
{
	if (const TWeakObjectPtr<UConsoleVariablesAsset> EditingAsset = GetEditingAsset();
		EditingAsset.IsValid() && ReferenceAssetOnDisk.IsValid())
	{
		UpdatePresetValuesForSave(EditingAsset.Get());
		
		if (UPackage* ReferencePackage = ReferenceAssetOnDisk.Get()->GetPackage())
		{
			ReferenceAssetOnDisk->CopyFrom(EditingAsset.Get());

			UEditorLoadingAndSavingUtils::SavePackages({ ReferencePackage }, false);

			return;
		}
	}

	// Fallback
	SaveCurrentPresetAs();
}

void FConsoleVariablesEditorMainPanel::SaveSpecificPreset(const TObjectPtr<UConsoleVariablesAsset> InPreset) const
{
	if (InPreset)
	{		
		if (UPackage* ReferencePackage = InPreset->GetPackage())
		{
			UEditorLoadingAndSavingUtils::SavePackages({ ReferencePackage }, false);

			return;
		}
	}

	// Fallback
	SaveSpecificPresetAs(InPreset);
}

void FConsoleVariablesEditorMainPanel::SaveCurrentPresetAs()
{
	if (const TWeakObjectPtr<UConsoleVariablesAsset> EditingAsset = GetEditingAsset(); EditingAsset.IsValid())
	{
		UpdatePresetValuesForSave(EditingAsset.Get());
			
		TArray<UObject*> SavedAssets;
		FEditorFileUtils::SaveAssetsAs({ EditingAsset.Get() }, SavedAssets);

		if (SavedAssets.Num())
		{
			UConsoleVariablesAsset* SavedAsset = Cast<UConsoleVariablesAsset>(SavedAssets[0]);

			if (ensure(SavedAsset))
			{
				ReferenceAssetOnDisk = SavedAsset;
			}
		}
	}
}

void FConsoleVariablesEditorMainPanel::SaveSpecificPresetAs(const TObjectPtr<UConsoleVariablesAsset> InPreset) const
{
	if (InPreset)
	{
		TArray<UObject*> SavedAssets;
		FEditorFileUtils::SaveAssetsAs({ InPreset }, SavedAssets);
	}
}

void FConsoleVariablesEditorMainPanel::ImportPreset(const FAssetData& InPresetAsset, const EConsoleVariablesEditorPresetImportMode InImportMode)
{
	FSlateApplication::Get().DismissAllMenus();
	
	if (UConsoleVariablesAsset* Preset = CastChecked<UConsoleVariablesAsset>(InPresetAsset.GetAsset()))
	{
		if (const TObjectPtr<UConsoleVariablesAsset> EditingAsset = GetEditingAsset(); ImportPreset_Impl(Preset, EditingAsset, InImportMode))
		{
			EditorList->RebuildList("", false);
		}
	}
}

void FConsoleVariablesEditorMainPanel::ImportPreset(const TObjectPtr<UConsoleVariablesAsset> InPreset)
{
	FSlateApplication::Get().DismissAllMenus();

	if (const TObjectPtr<UConsoleVariablesAsset> EditingAsset = GetEditingAsset(); ImportPreset_Impl(InPreset, EditingAsset, EConsoleVariablesEditorPresetImportMode::UseDefault))
	{
		EditorList->RebuildList("", false);
	}
}

bool FConsoleVariablesEditorMainPanel::ImportPreset_Impl(
	const TObjectPtr<UConsoleVariablesAsset> Preset, const TObjectPtr<UConsoleVariablesAsset> EditingAsset, const EConsoleVariablesEditorPresetImportMode InImportMode)
{
	if (Preset && EditingAsset)
	{
		bool bReplaceExisting = InImportMode == EConsoleVariablesEditorPresetImportMode::ReplaceExisting ? true : false;

		FConsoleVariablesEditorModule& CVarModule = GetConsoleVariablesModule();
		if (InImportMode == EConsoleVariablesEditorPresetImportMode::UseDefault)
		{
			if (UConsoleVariablesEditorProjectSettings* ProjectSettingsPtr =
				GetMutableDefault<UConsoleVariablesEditorProjectSettings>())
			{
				bReplaceExisting = ProjectSettingsPtr->PresetImportMode ==
					EConsoleVariablesEditorPresetImportMode::ReplaceExisting;
			}
		}

		if (bReplaceExisting || EditingAsset->GetSavedCommands().IsEmpty())
		{
			ReferenceAssetOnDisk = Preset;
		}
		else if (ReferenceAssetOnDisk.Get() != EditingAsset.Get())
		{
			ReferenceAssetOnDisk.Reset();
		}

		EditingAsset->Modify();

		if (bReplaceExisting)
		{
			// Reset the editing asset's variable values before replacing the list
			for (const FConsoleVariablesEditorAssetSaveData& CommandData : EditingAsset->GetSavedCommands())
			{
				const TWeakPtr<FConsoleVariablesEditorCommandInfo>& Command = CVarModule.FindCommandInfoByName(CommandData.CommandName);
				if (const TSharedPtr<FConsoleVariablesEditorCommandInfo> PinnedCommand = Command.Pin())
				{
					PinnedCommand->SetSourceFlag(PinnedCommand->StartupSource);
					PinnedCommand->ExecuteCommand(PinnedCommand->StartupValueAsString, true, false);
				}
			}

			EditingAsset->CopyFrom(Preset);
		}
		else
		{
			EditingAsset->AddFrom(Preset);
		}

		return true;
	}

	return false;
}

void FConsoleVariablesEditorMainPanel::OnConnectionChanged(EConcertConnectionStatus Status)
{
	switch (Status)
	{
		case EConcertConnectionStatus::Connected:
			UE_LOGF(LogConsoleVariablesEditor, VeryVerbose, "Multi-user has connected to a session.");
			break;
		case EConcertConnectionStatus::Disconnected:
			UE_LOGF(LogConsoleVariablesEditor, VeryVerbose, "Multi-user has disconnected from session.");
			break;
		default:
			break;
	}
}

void FConsoleVariablesEditorMainPanel::OnRemoteCvarChange(ERemoteCVarChangeType InChangeType, FString InName, FString InValue, EConsoleVariableFlags InFlags)
{
	FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();

	ConsoleVariablesEditorModule.OnRemoteCvarChanged(InChangeType, MoveTemp(InName), MoveTemp(InValue), InFlags);
}

void FConsoleVariablesEditorMainPanel::OnRemoteListItemCheckStateChange(const FString InName, ECheckBoxState InCheckedState)
{
	const FString CheckStateAsString =
		InCheckedState == ECheckBoxState::Checked ? "Checked" : InCheckedState == ECheckBoxState::Unchecked ? "Unchecked" : "Undetermined";
	UE_LOGF(LogConsoleVariablesEditor, Display,
		"%s: Remote check state change for list item with name '%ls' set to '%ls'", __FUNCTION__, *InName, *CheckStateAsString);
}

void FConsoleVariablesEditorMainPanel::SendListItemCheckStateChange(const FString& InName, ECheckBoxState InCheckedState)
{
	GetMultiUserManager().SendListItemCheckStateChange(InName, InCheckedState);
}
