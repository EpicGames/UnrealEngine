// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConsoleVariablesEditorModule.h"
#include "MultiUser/ConsoleVariableSync.h"
#include "Views/List/ConsoleVariablesEditorList.h"
#include "ConsoleVariablesEditorProjectSettings.h"

#include "Widgets/SWidget.h"


class FConsoleVariablesEditorList;
class SConsoleVariablesEditorMainPanel;
class UConsoleVariablesAsset;

class FConsoleVariablesEditorMainPanel : public TSharedFromThis<FConsoleVariablesEditorMainPanel>
{
public:
	FConsoleVariablesEditorMainPanel();

	~FConsoleVariablesEditorMainPanel();

	TSharedRef<SWidget> GetOrCreateWidget();

	static FConsoleVariablesEditorModule& GetConsoleVariablesModule();
	static TObjectPtr<UConsoleVariablesAsset> GetEditingAsset();

	void AddConsoleObjectToCurrentPreset(
		const FString InConsoleCommand, const FString InValue,
		const bool bScrollToNewRow = false) const;

	void UpdateCachedValue(const FString& InConsoleVar, const FString& Value);

	void OnRemoteCvarChanged();

	FReply ValidateConsoleInputAndAddToCurrentPreset(const FText& CommittedText) const;

	/*
	 * Regenerate the list items and refresh the list. Call when adding or removing variables.
	 * @param bShouldCacheValues If true, the current list's current values will be cached and then restored when the list is rebuilt. Otherwise preset values will be used.
	 */
	void RebuildList(const FString InConsoleCommandToScrollTo = "", bool bShouldCacheValues = true) const;

	/**
	 * Refresh filters and sorting.
	 * Useful for when the list state has gone stale but the variable count has not changed.
	 */
	void RefreshList() const;
	
	void UpdatePresetValuesForSave(TObjectPtr<UConsoleVariablesAsset> InAsset) const;

	void RefreshMultiUserDetails() const;

	// Save / Load

	void SaveCurrentPreset();
	void SaveSpecificPreset(const TObjectPtr<UConsoleVariablesAsset> InPreset) const;
	void SaveCurrentPresetAs();
	void SaveSpecificPresetAs(const TObjectPtr<UConsoleVariablesAsset> InPreset) const;
	void ImportPreset(const FAssetData& InPresetAsset, const EConsoleVariablesEditorPresetImportMode InImportMode = EConsoleVariablesEditorPresetImportMode::UseDefault);
	void ImportPreset(const TObjectPtr<UConsoleVariablesAsset> InPreset);

	TWeakObjectPtr<UConsoleVariablesAsset> GetReferenceAssetOnDisk() const
	{
		return ReferenceAssetOnDisk;
	}

	TWeakPtr<FConsoleVariablesEditorList> GetEditorList() const
	{
		return EditorList;
	}

	FConsoleVariablesEditorList::EConsoleVariablesEditorListMode GetEditorListMode() const
	{
		return EditorList->GetListMode();
	}

	UE::ConsoleVariables::MultiUser::Private::FManager& GetMultiUserManager()
	{
		return MultiUserManager;
	}

private:

	bool ImportPreset_Impl(const TObjectPtr<UConsoleVariablesAsset> Preset, const TObjectPtr<UConsoleVariablesAsset> EditingAsset, const EConsoleVariablesEditorPresetImportMode InImportMode);

	TSharedPtr<SConsoleVariablesEditorMainPanel> MainPanelWidget;

	// The non-transient loaded asset from which we will copy to the transient asset for editing
	TWeakObjectPtr<UConsoleVariablesAsset> ReferenceAssetOnDisk;

	TSharedPtr<FConsoleVariablesEditorList> EditorList;

	static void OnConnectionChanged(EConcertConnectionStatus Status);
	static void OnRemoteCvarChange(ERemoteCVarChangeType InChangeType, FString InName, FString InValue, EConsoleVariableFlags InFlags);
	static void OnRemoteListItemCheckStateChange(const FString InName, ECheckBoxState InCheckedState);
	
	void SendListItemCheckStateChange(const FString& InName, ECheckBoxState InCheckedState);
	
	UE::ConsoleVariables::MultiUser::Private::FManager MultiUserManager;
	FDelegateHandle OnConnectionChangedHandle;
	FDelegateHandle OnRemoteCVarChangeHandle;
	FDelegateHandle OnRemoteListItemCheckStateChangeHandle;
};
