// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/MainPanel/SConsoleVariablesEditorMainPanel.h"

#include "ConsoleVariablesAsset.h"
#include "ConsoleVariablesEditorLog.h"
#include "ConsoleVariablesEditorModule.h"
#include "ConsoleVariablesEditorProjectSettings.h"
#include "ConsoleVariablesEditorStyle.h"
#include "MultiUser/ConsoleVariableSyncData.h"
#include "Views/List/ConsoleVariablesEditorList.h"
#include "Views/MainPanel/ConsoleVariablesEditorMainPanel.h"
#include "Views/Widgets/SConsoleVariablesEditorCustomConsoleInputBox.h"

#include "ContentBrowserModule.h"
#include "DetailsViewArgs.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SPositiveActionButton.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

void SConsoleVariablesEditorMainPanel::Construct(
	const FArguments& InArgs, const TSharedRef<FConsoleVariablesEditorMainPanel>& InMainPanel)
{
	check(InMainPanel->GetEditorList().IsValid());

	MainPanel = InMainPanel;
	
	ChildSlot
	[
		SNew(SVerticalBox)
			
		+SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		.Padding(FMargin(8.f, 0.f, 8.f, 0.f))
		[
			GeneratePanelToolbar()
		]

		+SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		.Padding(FMargin(8.f, 0.f, 8.f, 0.f))
		[
			SAssignNew(AddConsoleObjectInputBoxPtr, SConsoleVariablesEditorCustomConsoleInputBox)
			.Visibility(EVisibility::Collapsed)
			.OnTextCommitted(this, &SConsoleVariablesEditorMainPanel::ValidateConsoleInputAndAddToCurrentPreset)
		]

		+SVerticalBox::Slot()
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			.ResizeMode(ESplitterResizeMode::FixedSize)

			+SSplitter::Slot()
			[
				MainPanel.Pin()->GetEditorList().Pin()->GetOrCreateWidget()
			]

			+SSplitter::Slot()
			[
				SAssignNew(MultiUserDetailsBox, SVerticalBox)
				.Visibility(EVisibility::Collapsed)
			]
		]
	];

	CreateConcertButtonIfNeeded();
}

SConsoleVariablesEditorMainPanel::~SConsoleVariablesEditorMainPanel()
{
	MainPanel.Reset();
	ToolbarHBox.Reset();
	ConcertButtonPtr.Reset();
	MultiUserDetailsBox.Reset();
}

void SConsoleVariablesEditorMainPanel::ValidateConsoleInputAndAddToCurrentPreset(const FText& CommittedText) const
{
	MainPanel.Pin()->ValidateConsoleInputAndAddToCurrentPreset(CommittedText);
}

void SConsoleVariablesEditorMainPanel::RefreshMultiUserDetails() const
{
	UConcertCVarSynchronization* CVarSync = GetMutableDefault<UConcertCVarSynchronization>();

	UConcertCVarConfig* CVarConfig = GetMutableDefault<UConcertCVarConfig>();

	MultiUserDetailsBox->ClearChildren();

	MultiUserDetailsBox->AddSlot()
	.AutoHeight()
	[
		GetConcertDetailsWidget(CVarSync)
	];

	MultiUserDetailsBox->AddSlot()
	[
		GetConcertDetailsWidget(CVarConfig)
	];
}

void SConsoleVariablesEditorMainPanel::ToggleMultiUserDetails(ECheckBoxState CheckState) const
{
	const bool bShouldBeVisible = CheckState == ECheckBoxState::Checked;
	MultiUserDetailsBox->SetVisibility(bShouldBeVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed);

	if (bShouldBeVisible)
	{
		RefreshMultiUserDetails();
	}
}

TSharedRef<SWidget> SConsoleVariablesEditorMainPanel::GeneratePanelToolbar()
{
	return SAssignNew(ToolbarHBox, SHorizontalBox)
				
			// Add Console Variable button
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Left)
			.Padding(FMargin(0, 4))
			[
				SNew(SPositiveActionButton)
				.Text(LOCTEXT("ConsoleVariable", "Console Variable"))
				.OnClicked(this, &SConsoleVariablesEditorMainPanel::HandleAddConsoleVariableButtonClicked)
			]

			// Presets Management Button
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(FMargin(8.f, 4, 0.f, 4))
			[
				SNew(SComboButton)
				.ToolTipText(LOCTEXT("PresetManagementButton_Tooltip", "Export the current CVar list to a preset, or import a copy of an existing preset."))
				.ContentPadding(FMargin(4, 0.5f))
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton"))
				.OnGetMenuContent(this, &SConsoleVariablesEditorMainPanel::OnGeneratePresetsMenu)
				.ForegroundColor(FStyleColors::Foreground)
				.ButtonContent()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.Padding(0, 1, 4, 0)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("AssetEditor.SaveAsset"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]

					+ SHorizontalBox::Slot()
					.Padding(0, 1, 0, 0)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("PresetsToolbarButton", "Presets"))
					]
				]
			];
}

FReply SConsoleVariablesEditorMainPanel::HandleAddConsoleVariableButtonClicked()
{
	if (AddConsoleObjectInputBoxPtr.IsValid())
	{
		AddConsoleObjectInputBoxPtr->SetVisibility(EVisibility::Visible);

		AddConsoleObjectInputBoxPtr->TakeKeyboardFocus();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SConsoleVariablesEditorMainPanel::CreateConcertButtonIfNeeded()
{
	if (MainPanel.Pin()->GetMultiUserManager().IsInitialized())
	{		
		// Toggle Multi-User Details
		ToolbarHBox->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(11.f, 1.f, 1.f, 1.f)
		[
			SAssignNew(ConcertButtonPtr, SCheckBox)
			.Padding(4.f)
			.ToolTipText(LOCTEXT("ShowConcertSettings_Tip", "Show the multi-user controls for Console Variables"))
			.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
			.ForegroundColor(FSlateColor::UseForeground())
			.IsChecked(false)
			.OnCheckStateChanged_Raw(this, &SConsoleVariablesEditorMainPanel::ToggleMultiUserDetails)
			[
				SNew(SImage)
				.Image(FConsoleVariablesEditorStyle::Get().GetBrush("Icons.MultiUser"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}
}

TSharedRef<SWidget> SConsoleVariablesEditorMainPanel::OnGeneratePresetsMenu()
{
	FMenuBuilder MenuBuilder(true, NULL);

	IContentBrowserSingleton& ContentBrowser =
		FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	const FText LoadedPresetName = MainPanel.Pin()->GetReferenceAssetOnDisk().IsValid() ?
		FText::Format(
			LoadedPresetFormatText,
			FText::FromString(MainPanel.Pin()->GetReferenceAssetOnDisk()->GetName())) : NoLoadedPresetText;

	FUIAction PresetNameAction = FUIAction();
	PresetNameAction.CanExecuteAction = FCanExecuteAction::CreateLambda([]() { return false; });
	
	MenuBuilder.AddMenuEntry(
		LoadedPresetName,
		LoadedPresetName,
		FSlateIcon(),
		PresetNameAction,
		NAME_None,
		EUserInterfaceActionType::None
	);

	MenuBuilder.AddMenuSeparator();
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("SavePreset_Text", "Save Preset"),
		LOCTEXT("SavePreset_Tooltip", "Save the current preset if one has been loaded. Otherwise, the Save As dialog will be opened."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAsset"),
		FUIAction(FExecuteAction::CreateRaw(MainPanel.Pin().Get(), &FConsoleVariablesEditorMainPanel::SaveCurrentPreset)),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SavePresetAs_Text", "Save Preset As"),
		LOCTEXT("SavePresetAs_Tooltip", "Save the current configuration as a new preset that can be shared between multiple jobs, or imported later as the base of a new configuration."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAssetAs"),
		FUIAction(FExecuteAction::CreateRaw(MainPanel.Pin().Get(), &FConsoleVariablesEditorMainPanel::SaveCurrentPresetAs)),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bShowBottomToolbar = true;
		AssetPickerConfig.bAutohideSearchBar = false;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bShowPathInColumnView = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.bSortByPathInColumnView = false;
		AssetPickerConfig.bForceShowEngineContent = false;
		AssetPickerConfig.bForceShowPluginContent = false;

		AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoPresets_Warning", "No Presets Found");
		AssetPickerConfig.Filter.ClassPaths.Add(UConsoleVariablesAsset::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda(
			[this](const FAssetData& InAssetData)
			{
				if (MainPanel.IsValid())
				{
					MainPanel.Pin()->ImportPreset(InAssetData);
				}
			});
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ImportPreset_MenuSection", "Import Preset"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ImportPreset_ImportMode_Add", "Add Preset to Existing Console Variables"),
			LOCTEXT(
				"ImportPreset_ImportMode_Add_Tooltip",
				"Add the list of variables from the imported preset to the current preset, replacing the "
				"values of any overlapping variables with the values from the imported preset."
			),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]() {
					if (UConsoleVariablesEditorProjectSettings* ProjectSettingsPtr =
						GetMutableDefault<UConsoleVariablesEditorProjectSettings>())
					{
						ProjectSettingsPtr->PresetImportMode =
							EConsoleVariablesEditorPresetImportMode::AddToExisting;
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() {
					if (const UConsoleVariablesEditorProjectSettings* ProjectSettingsPtr =
						GetDefault<UConsoleVariablesEditorProjectSettings>())
					{
						return ProjectSettingsPtr->PresetImportMode ==
							EConsoleVariablesEditorPresetImportMode::AddToExisting;
					}

					return false;
				})
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ImportPreset_ImportMode_Replace", "Replace Console Variables with Preset"),
			LOCTEXT(
				"ImportPreset_ImportMode_Replace_Tooltip",
				"Completely replace the list of variables in the current preset, resetting them "
				"to their default values and removing them from the list before importing the new preset's variable list."
			),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]() {
					if (UConsoleVariablesEditorProjectSettings* ProjectSettingsPtr =
						GetMutableDefault<UConsoleVariablesEditorProjectSettings>())
					{
						ProjectSettingsPtr->PresetImportMode =
							EConsoleVariablesEditorPresetImportMode::ReplaceExisting;
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() {
					if (const UConsoleVariablesEditorProjectSettings* ProjectSettingsPtr =
						GetDefault<UConsoleVariablesEditorProjectSettings>())
					{
						return ProjectSettingsPtr->PresetImportMode ==
							EConsoleVariablesEditorPresetImportMode::ReplaceExisting;
					}

					return false;
				})
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuSeparator();

		TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.MinDesiredWidth(400.f)
			.MinDesiredHeight(400.f)
			[
				ContentBrowser.CreateAssetPicker(AssetPickerConfig)
			];

		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	TSharedRef<SWidget> Widget = MenuBuilder.MakeWidget();
	FChildren* ChildWidgets = Widget->GetChildren();
	for (int32 ChildItr = 0; ChildItr < ChildWidgets->Num(); ChildItr++)
	{
		TSharedRef<SWidget> Child = ChildWidgets->GetChildAt(ChildItr);

		Child->EnableToolTipForceField(false);
	}
	Widget->EnableToolTipForceField(false);
	
	return Widget;
}

TSharedRef<SWidget> SConsoleVariablesEditorMainPanel::GetConcertDetailsWidget(UObject* InObject)
{
	FPropertyEditorModule& PropertyEditorModule =
		FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowScrollBar = false;

	TSharedRef<IDetailsView> Details = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	Details->SetObjects(TArray{InObject});

	return Details;
}

#undef LOCTEXT_NAMESPACE
