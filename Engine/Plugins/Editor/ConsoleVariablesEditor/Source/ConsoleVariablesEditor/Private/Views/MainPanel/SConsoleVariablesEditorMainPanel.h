// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SCompoundWidget.h"

class FConsoleVariablesEditorMainPanel;
class SBox;
class SConsoleVariablesEditorCustomConsoleInputBox;
class SHorizontalBox;
class SVerticalBox;
class UConsoleVariablesAsset;

class SConsoleVariablesEditorMainPanel final : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS(SConsoleVariablesEditorMainPanel)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FConsoleVariablesEditorMainPanel>& InMainPanel);

	virtual ~SConsoleVariablesEditorMainPanel() override;

	/** A handler for when text is committed to the Console Input Box that appears when clicking the "Add Console Variable" button */
	void ValidateConsoleInputAndAddToCurrentPreset(const FText& CommittedText) const;

	void RefreshMultiUserDetails() const;
	void ToggleMultiUserDetails(ECheckBoxState CheckState) const;

private:

	/** A reference to the struct that controls this widget */
	TWeakPtr<FConsoleVariablesEditorMainPanel> MainPanel;

	TSharedPtr<SConsoleVariablesEditorCustomConsoleInputBox> AddConsoleObjectInputBoxPtr;

	TSharedPtr<SHorizontalBox> ToolbarHBox;

	/** A reference to the button which opens the plugin settings */
	TSharedPtr<SCheckBox> ConcertButtonPtr;

	/** Creates the toolbar at the top of the MainPanel widget */
	TSharedRef<SWidget> GeneratePanelToolbar();

	FReply HandleAddConsoleVariableButtonClicked();
	
	void CreateConcertButtonIfNeeded();

	// Save / Load

	#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

	const FText NoLoadedPresetText = LOCTEXT("NoLoadedPreset", "No Loaded Preset");
	const FText LoadedPresetFormatText = LOCTEXT("LoadedPresetFormat", "Current Preset: {0}");

    #undef LOCTEXT_NAMESPACE

	/** Creates a special asset picker widget to display when the Save/Load button is clicked */
	TSharedRef<SWidget> OnGeneratePresetsMenu();

	TSharedPtr<SVerticalBox> MultiUserDetailsBox;

	/** Generates a customized details widget given an object. Intended for multi-user settings display. */
	static TSharedRef<SWidget> GetConcertDetailsWidget(UObject* InObject);
};
