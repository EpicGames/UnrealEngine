// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConsoleVariablesEditorGlobalSearchToggle.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

void SConsoleVariablesEditorGlobalSearchToggle::Construct(const FArguments& Args, const FText& InButtonText)
{
	OnToggleCtrlClicked = Args._OnToggleCtrlClicked;
	OnToggleAltClicked = Args._OnToggleAltClicked;
	OnToggleMiddleButtonClicked = Args._OnToggleMiddleButtonClicked;
	OnToggleRightButtonClicked = Args._OnToggleRightButtonClicked;
	OnToggleClickedOnce = Args._OnToggleClickedOnce;

	GlobalSearchText = InButtonText;
	CheckedColor = USlateThemeManager::Get().GetColor(EStyleColor::Primary);
	UncheckedColor = USlateThemeManager::Get().GetColor(EStyleColor::Black);
	
	ChildSlot
    [
		SNew(SBorder)
		.Padding(1.0f)
		.BorderImage(FAppStyle::Get().GetBrush("ContentBrowser.FilterBackground"))
		 [
			SAssignNew( ToggleButtonPtr, SCheckBox )
			.Style(FAppStyle::Get(), "ContentBrowser.FilterButton")
			.ToolTipText(LOCTEXT("GlobalSearchPillButtonTooltip","Toggle this search"))
			.Padding(0.0f)
			.IsChecked(this, &SConsoleVariablesEditorGlobalSearchToggle::GetCheckedState)
			.OnCheckStateChanged_Lambda(
				[this](ECheckBoxState NewCheckboxState)
			{
				SetIsButtonChecked(!GetIsToggleChecked());

				if (FSlateApplication::Get().GetModifierKeys().IsControlDown() && OnToggleCtrlClicked.IsBound())
				{
					SetIsMarkedForDelete(true);
					OnToggleCtrlClicked.Execute();
				}
				else if (FSlateApplication::Get().GetModifierKeys().IsAltDown() && OnToggleAltClicked.IsBound())
				{
					SetIsMarkedForDelete(true);
					OnToggleAltClicked.Execute();
				}
				else if (OnToggleClickedOnce.IsBound())
				{
					SetIsMarkedForDelete(false);
					OnToggleClickedOnce.Execute();
				}
			})
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("ContentBrowser.FilterImage"))
					.ColorAndOpacity_Lambda([this]()
					{
						return GetIsToggleChecked() ? CheckedColor : UncheckedColor;
					})
				]
				+SHorizontalBox::Slot()
				.Padding(2.f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::Format(INVTEXT("\"{0}\""), GlobalSearchText)) // Show text in quotes
				]
			]
		]
    ];
}

SConsoleVariablesEditorGlobalSearchToggle::~SConsoleVariablesEditorGlobalSearchToggle()
{
	OnToggleCtrlClicked.Unbind();
	OnToggleAltClicked.Unbind();
	OnToggleMiddleButtonClicked.Unbind();
	OnToggleRightButtonClicked.Unbind();
	OnToggleClickedOnce.Unbind();

	ToggleButtonPtr.Reset();
}

FReply SConsoleVariablesEditorGlobalSearchToggle::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled();
}

FReply SConsoleVariablesEditorGlobalSearchToggle::OnMouseButtonUp( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) 
{
	if(InMouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton && OnToggleMiddleButtonClicked.IsBound())
	{
		SetIsMarkedForDelete(true);
		return OnToggleMiddleButtonClicked.Execute().ReleaseMouseCapture();
	}
	else if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton && OnToggleRightButtonClicked.IsBound())
	{
		SetIsMarkedForDelete(true);
		return OnToggleRightButtonClicked.Execute().ReleaseMouseCapture();
	}
		
	return FReply::Handled().ReleaseMouseCapture();
}

bool SConsoleVariablesEditorGlobalSearchToggle::GetIsToggleChecked() const
{
	return bIsToggleChecked;
}

ECheckBoxState SConsoleVariablesEditorGlobalSearchToggle::GetCheckedState() const
{
	return GetIsToggleChecked() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SConsoleVariablesEditorGlobalSearchToggle::SetIsButtonChecked(const bool bNewIsButtonChecked)
{
	bIsToggleChecked = bNewIsButtonChecked;
}

bool SConsoleVariablesEditorGlobalSearchToggle::GetIsMarkedForDelete() const
{
	return bIsMarkedForDelete;
}

void SConsoleVariablesEditorGlobalSearchToggle::SetIsMarkedForDelete(const bool bNewMark)
{
	bIsMarkedForDelete = bNewMark;
}

#undef LOCTEXT_NAMESPACE
