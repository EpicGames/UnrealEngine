// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SCompoundWidget.h"

class SConsoleVariablesEditorGlobalSearchToggle : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SConsoleVariablesEditorGlobalSearchToggle)
	{}
		SLATE_EVENT(FOnClicked, OnToggleCtrlClicked)
		SLATE_EVENT(FOnClicked, OnToggleAltClicked)
		SLATE_EVENT(FOnClicked, OnToggleMiddleButtonClicked)
		SLATE_EVENT(FOnClicked, OnToggleRightButtonClicked)
		SLATE_EVENT(FOnClicked, OnToggleClickedOnce)
	SLATE_END_ARGS();

	void Construct(const FArguments& Args, const FText& InButtonText);

	virtual ~SConsoleVariablesEditorGlobalSearchToggle() override;
	
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;

	
	[[nodiscard]] bool GetIsToggleChecked() const;
	[[nodiscard]] ECheckBoxState GetCheckedState() const;
	void SetIsButtonChecked(const bool bNewIsButtonChecked);

	[[nodiscard]] bool GetIsMarkedForDelete() const;
	void SetIsMarkedForDelete(const bool bNewMark);

	const FText& GetGlobalSearchText()
	{
		return GlobalSearchText;
	}

private:
	
	FOnClicked OnToggleCtrlClicked;
	FOnClicked OnToggleAltClicked;
	FOnClicked OnToggleMiddleButtonClicked;
	FOnClicked OnToggleRightButtonClicked;
	FOnClicked OnToggleClickedOnce;

	TSharedPtr<SCheckBox> ToggleButtonPtr;
	bool bIsToggleChecked = true;

	/** When clicked in a special manner, this search button will be marked for deletion
	 * The responsibility is on external classes to remove the button from the UI the external class controls. */
	bool bIsMarkedForDelete = false;

	FLinearColor CheckedColor;
	FLinearColor UncheckedColor;

	FText GlobalSearchText;
};
