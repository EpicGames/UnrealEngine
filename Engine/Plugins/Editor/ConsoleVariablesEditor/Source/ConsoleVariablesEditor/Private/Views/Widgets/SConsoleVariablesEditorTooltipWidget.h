// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class SConsoleVariablesEditorTooltipWidget : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SConsoleVariablesEditorTooltipWidget)
	{}

	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const FString InCommandName, const FString InHelpString)
	{
		TSharedPtr<SVerticalBox> VerticalBox;
		
		ChildSlot
		[
			SAssignNew(VerticalBox, SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 4)
			[
				SNew(SBorder)
				.Padding(6)
				.BorderImage(FAppStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
				[
					SNew(STextBlock)
					.Text(FText::FromString(InCommandName))
					.Font(FAppStyle::Get().GetFontStyle("ContentBrowser.TileViewTooltip.NameFont"))
					.AutoWrapText(true)
				]
			]
		];

		if (!InHelpString.IsEmpty())
		{
			VerticalBox->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Padding(6)
				.BorderImage(FAppStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
				[
					SNew(STextBlock)
					.Text(FText::FromString(InHelpString))
					.AutoWrapText(true)
				]
			];
		}
	}

	static TSharedRef<SToolTip> MakeTooltip(const FString InCommandName, const FString InHelpString)
	{
		return
			SNew(SToolTip)
			.BorderImage(FAppStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.NonContentBorder"))
			[
				SNew(SBox)
				.WidthOverride(TooltipWindowWidth)
				[
					SNew(SConsoleVariablesEditorTooltipWidget, InCommandName, InHelpString)
				]
			];
	}

private:

	static constexpr float TooltipWindowWidth = 500.f;
};
