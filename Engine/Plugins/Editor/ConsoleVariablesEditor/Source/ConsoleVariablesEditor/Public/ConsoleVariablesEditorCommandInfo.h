// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/IConsoleManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/DateTime.h"
#include "UObject/ObjectMacros.h"

#define UE_API CONSOLEVARIABLESEDITOR_API

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

struct FConsoleVariablesEditorCommandInfo
{
	enum class EConsoleObjectType
	{
		// A console command that has no associated console object but is parsed externally, e.g. 'stat unit'
		NullObject,
		// A console command with an associated console object, like 'r.SetNearClipPlane'
		Command,
		// A console variable such as 'r.ScreenPercentage'
		Variable
	};
	
	struct FStaticConsoleVariableFlagInfo
	{
		EConsoleVariableFlags Flag;
		FText DisplayText;
	};
	
	FConsoleVariablesEditorCommandInfo(const FString& InCommand)
	: Command(InCommand)
	{
		if (GetConsoleObjectPtr())
		{
			ObjectType = EConsoleObjectType::Command;
			
			if (const IConsoleVariable* AsVariable = GetConsoleVariablePtr())
			{
				ObjectType = EConsoleObjectType::Variable;
				StartupValueAsString = AsVariable->GetString();
				StartupSource = GetSource();
			}
		}
	}
	
	~FConsoleVariablesEditorCommandInfo()
	{
		if (!IsEngineExitRequested()) // CrashFix as GetConsoleVariablePtr() returns stale deleted IConsoleVariable during shut down
		{
			if (IConsoleVariable* AsVariable = GetConsoleVariablePtr())
			{
				AsVariable->OnChangedDelegate().Remove(OnVariableChangedCallbackHandle);
			}
		}
	}
	
	FORCEINLINE bool operator==(const FConsoleVariablesEditorCommandInfo& Comparator) const
	{
		return Command.Equals(Comparator.Command, ESearchCase::IgnoreCase);
	}
	
	void SetIfChangedInCurrentPreset(const bool bNewSetting)
	{
		bSetInCurrentSession = bNewSetting;
	}
	
	/**
	  * Sets a variable to the specified value whilst maintaining its SetBy flag.
	  * Non-variables will be executed through the console.
	  * @param bShouldTransactInConcert If true, send this value to multi-user.
	  * @param bSetInSession If true, this CommandInfo's associated variable row will display "Session" in the UI.
	  * @param bIsInteractiveChange If true, don't print the command or send it to multi-user/concert.
	 */
	UE_API void ExecuteCommand(
		const FString& NewValueAsString, const bool bShouldTransactInConcert = true,
		const bool bSetInSession = true, const bool bIsInteractiveChange = false);

	/**
	 * Prints the current value or state of the command, similar to the text generated in teh output log when entering a command manually.
	 */
	UE_API void PrintCommandOrVariable();
 
	/** Get a reference to the cached console object. May return nullptr if unregistered. */
	UE_API IConsoleObject* GetConsoleObjectPtr();

	/** Return the console object as a console variable object if applicable. May return nullptr if unregistered. */
	UE_API IConsoleVariable* GetConsoleVariablePtr();

	/**
	 *Return the console object as a console command object if applicable.
	 *Does not consider externally parsed console commands, as they have no associated objects.
	 */
	UE_API IConsoleCommand* GetConsoleCommandPtr();
 
	static UE_API UWorld* GetCurrentWorld();

	UE_API FString GetHelpText();
 
	UE_API EConsoleVariableFlags GetSource();
 
	UE_API void ClearSourceFlags();

	UE_API void SetSourceFlag(const EConsoleVariableFlags InSource);
 
	UE_API FText GetSourceAsText();

	static UE_API FText ConvertConsoleVariableSetByFlagToText(const EConsoleVariableFlags InFlag);

	/**
	 * If this console object is a variable, get its value as string via OutValueAsString.
	 * If it's not a variable type (such as a command) then this function will return false.
	 */
	UE_API bool GetCurrentValueAsString(FString& OutValueAsString);
 
	UE_API bool IsCurrentValueDifferentFromInputValue(const FString& InValueToCompare);
 
	UE_API void OnConsoleVariableChanged(IConsoleVariable* ChangedVariable);

	/** The actual string key or name */
	UPROPERTY()
	FString Command;
	EConsoleObjectType ObjectType = EConsoleObjectType::NullObject;
	/** This object is periodically refreshed to mitigate the occurrence of stale pointers. */
	IConsoleObject* ConsoleObjectPtr = nullptr;
	
	/** The value of this variable (if Variable object type) when the module started in this session after it may have been set by an ini file. */
	FString StartupValueAsString;
	
	/** The source of this variable's (if Variable object type) last setting as recorded when the plugin was loaded. */
	EConsoleVariableFlags StartupSource = ECVF_Default;
	/** If the variable was last changed by the current preset */
	bool bSetInCurrentSession = false;
	bool bHasAttemptedFind = false;
	/** When variables change, this callback is executed. */
	FDelegateHandle OnVariableChangedCallbackHandle;
	/** A mapping of SetBy console variable flags to information like the associated display text. */
	static const inline TArray<FStaticConsoleVariableFlagInfo> SupportedFlags =
	{
		{ EConsoleVariableFlags::ECVF_SetByConstructor, LOCTEXT("Source_SetByConstructor", "Constructor") },
		{ EConsoleVariableFlags::ECVF_SetByScalability, LOCTEXT("Source_SetByScalability", "Scalability") },
		{ EConsoleVariableFlags::ECVF_SetByGameSetting, LOCTEXT("Source_SetByGameSetting", "Game Setting") },
		{ EConsoleVariableFlags::ECVF_SetByProjectSetting, LOCTEXT("Source_SetByProjectSetting", "Project Setting") },
		{ EConsoleVariableFlags::ECVF_SetBySystemSettingsIni, LOCTEXT("Source_SetBySystemSettingsIni", "System Settings ini") },
		{ EConsoleVariableFlags::ECVF_SetByDeviceProfile, LOCTEXT("Source_SetByDeviceProfile", "Device Profile") },
		{ EConsoleVariableFlags::ECVF_SetByGameOverride, LOCTEXT("Source_SetByGameOverride", "Game Override") },
		{ EConsoleVariableFlags::ECVF_SetByConsoleVariablesIni, LOCTEXT("Source_SetByConsoleVariablesIni", "Console Variables ini") },
		{ EConsoleVariableFlags::ECVF_SetByCommandline, LOCTEXT("Source_SetByCommandline", "Command line") },
		{ EConsoleVariableFlags::ECVF_SetByCode, LOCTEXT("Source_SetByCode", "Code") },
		{ EConsoleVariableFlags::ECVF_SetByConsole, LOCTEXT("Source_SetByConsole", "Console") },
		{ EConsoleVariableFlags::ECVF_SetByPreview, LOCTEXT("Source_SetByPreview", "Preview") }
	};
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
