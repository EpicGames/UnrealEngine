// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "ConsoleVariablesEditorProjectSettings.h"

#include "ConsoleVariablesEditorFunctionLibrary.generated.h"

#define UE_API CONSOLEVARIABLESEDITOR_API

class FString;
class UConsoleVariablesAsset;

/** An asset used to track collections of console variables that can be recalled and edited using the Console Variables Editor. */
UCLASS(MinimalAPI, BlueprintType)
class UConsoleVariablesEditorFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/** Return the currently loaded list of variables in the Console Variables Editor. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	static UE_API UConsoleVariablesAsset* GetCurrentlyLoadedPreset();

	/** Loads the given asset in the Console Variables Editor and sets all its variable values. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	static UE_API void LoadPresetIntoConsoleVariablesEditor(const UConsoleVariablesAsset* InAsset, const EConsoleVariablesEditorPresetImportMode InImportMode = EConsoleVariablesEditorPresetImportMode::AddToExisting);

	/*
	 * Saves the current list in the Console Variables Editor to the given asset.
	 * The Asset will not be automatically saved.
	 * @return true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	static UE_API bool CopyCurrentListToAsset(UConsoleVariablesAsset* InAsset);

	/*
	 * Adds a validated command to the current preset with its current value.
	 * The Asset will not be automatically saved.
	 * @return true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	static UE_API bool AddValidatedCommandToCurrentPreset(const FString NewCommand);

	/*
	 * Removes a command from the current preset if it exists in the saved data.
	 * The Asset will not be automatically saved.
	 * @return true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	static UE_API bool RemoveCommandFromCurrentPreset(const FString NewCommand);

	/** Return an array of strings containing the command names for each command found in the given preset. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	static UE_API bool GetListOfCommandsFromPreset(const UConsoleVariablesAsset* InAsset, TArray<FString>& OutCommandList);

	/** Set a console variable value directly. Returns true if the console object exists. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor", meta=(DisplayName="Set Console Variable By Name (Float)"))
	static UE_API bool SetConsoleVariableByName_Float(const FString InCommandName, const float InValue);

	/** Set a console variable value directly. Returns true if the console object exists. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor", meta=(DisplayName="Set Console Variable By Name (Bool)"))
	static UE_API bool SetConsoleVariableByName_Bool(const FString InCommandName, const bool InValue);

	/** Set a console variable value directly. Returns true if the console object exists. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor", meta=(DisplayName="Set Console Variable By Name (Int)"))
	static UE_API bool SetConsoleVariableByName_Int(const FString InCommandName, const int32 InValue);
	
	/** Set a console variable value directly. Returns true if the console object exists. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor", meta=(DisplayName="Set Console Variable By Name (String)"))
	static UE_API bool SetConsoleVariableByName_String(const FString InCommandName, const FString InValue);

	/** Get a console variable's string value directly. Returns true if the console object exists. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	static UE_API bool GetConsoleVariableStringValue(const FString InCommandName, FString& OutValue);

	/** Set a console variable value directly. Returns true if the console object exists. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	static UE_API bool GetConsoleVariableSourceByName(const FString InCommandName, FString& OutValue);

	/** Return whether the Multi-user sync setting for the current instance of the editor is enabled. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	static UE_API bool GetEnableMultiUserCVarSync();
	
	/** Enable or disable the Multi-user sync setting for the current instance of the editor. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	static UE_API void SetEnableMultiUserCVarSync(bool bNewSetting);
	
};

#undef UE_API
