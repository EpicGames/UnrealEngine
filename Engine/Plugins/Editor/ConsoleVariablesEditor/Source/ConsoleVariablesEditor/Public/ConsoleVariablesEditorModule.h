// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleInterface.h"
#include "Widgets/Docking/SDockTab.h"
#include "ConsoleVariablesEditorProjectSettings.h"

struct FAssetData;
struct FConsoleVariablesEditorCommandInfo;

class FConsoleVariablesEditorMainPanel;
class FConsoleVariablesEditorToolkit;
class ISettingsSection;
class UConsoleVariablesAsset;

enum class ERemoteCVarChangeType : uint8;

class FConsoleVariablesEditorModule : public IModuleInterface, public FGCObject
{
public:
	
	CONSOLEVARIABLESEDITOR_API static FConsoleVariablesEditorModule& Get();

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void PreUnloadCallback() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

	void SavePreset() const;
	void SaveSpecificPreset(const TObjectPtr<UConsoleVariablesAsset> Preset) const;
	void SavePresetAs() const;
	void SaveSpecificPresetAs(const TObjectPtr<UConsoleVariablesAsset> Preset) const;
	void OpenConsoleVariablesDialogWithPreset(const TObjectPtr<UConsoleVariablesAsset> Preset) const;
	void OpenConsoleVariablesDialogWithAssetSelected(const FAssetData& InAssetData, const EConsoleVariablesEditorPresetImportMode InImportMode = EConsoleVariablesEditorPresetImportMode::UseDefault) const;

	/** Find all console variables and cache their startup values */
	void QueryAndBeginTrackingConsoleVariables();

	void AddConsoleObjectCommandInfoToMainReference(TSharedRef<FConsoleVariablesEditorCommandInfo> InCommandInfo);

	/** Find a tracked console variable by the command string with optional case sensitivity. */
	CONSOLEVARIABLESEDITOR_API TWeakPtr<FConsoleVariablesEditorCommandInfo> FindCommandInfoByName(
		const FString& NameToSearch, ESearchCase::Type InSearchCase = ESearchCase::IgnoreCase);

	/*Find all tracked console variables matching a specific search query with optional case sensitivity.
	 *Individual members of InTokens will be considered "AnyOf" or "OR" searches. If SearchTerms contains any individual member it will match.
	 *Members will be tested for a space character (" "). If a space is found, a subsearch will be run.
	 *This subsearch will be an "AllOf" or "AND" type search in which all strings, separated by a space, must be found in the search terms.
	 */
	CONSOLEVARIABLESEDITOR_API TArray<TWeakPtr<FConsoleVariablesEditorCommandInfo>> FindCommandInfosMatchingTokens(
		const TArray<FString>& InTokens, ESearchCase::Type InSearchCase = ESearchCase::IgnoreCase);

	[[nodiscard]] TObjectPtr<UConsoleVariablesAsset> GetPresetAsset() const;
	[[nodiscard]] TObjectPtr<UConsoleVariablesAsset> GetGlobalSearchAsset() const;
	
	FReply ValidateConsoleInputAndAddToCurrentPreset(const FText& CommittedText) const;
	/*
	 * Regenerate the list items and refresh the list. Call when adding or removing variables.
	 * @param bShouldCacheValues If true, the current list's current values will be cached and then restored when the list is rebuilt. Otherwise preset values will be used.
	 */
	void RebuildList(const FString InConsoleCommandToScrollTo = "", bool bShouldCacheValues = true) const;
	
	/**
	 * Refresh filters and sorting.
	 * Useful for when the list state has gone stale but the variable count has not changed.
	 */
	void RefreshList() const;
	
	void UpdatePresetValuesForSave(TObjectPtr<UConsoleVariablesAsset> InAsset);

	/** Fills Global Search Asset's Saved Commands with variables matching the specified query. Returns false if no matches were found. */
	bool PopulateGlobalSearchAssetWithVariablesMatchingTokens(const TArray<FString>& InTokens);

	void SendMultiUserConsoleVariableChange(ERemoteCVarChangeType InChangeType, const FString& InVariableName, const FString& InValueAsString, EConsoleVariableFlags InFlags);
	void OnRemoteCvarChanged(ERemoteCVarChangeType InChangeType, FString InName, FString InValue, EConsoleVariableFlags InFlags);

	virtual void AddReferencedObjects( FReferenceCollector& Collector )  override;
	virtual FString GetReferencerName() const override;

	void OnConsoleVariableChanged(FConsoleVariablesEditorCommandInfo& CommandInfo, IConsoleVariable* ChangedVariable);

private:
	/** Scoped based struct to track inbound cvars set by Multi-user. */
	struct FScopeMultiUserReceiveCVar
	{
		FScopeMultiUserReceiveCVar() = delete;
		FScopeMultiUserReceiveCVar(const FScopeMultiUserReceiveCVar&) = delete;
		FScopeMultiUserReceiveCVar& operator=(const FScopeMultiUserReceiveCVar&) = delete;

		FScopeMultiUserReceiveCVar(FString InCommand, TMap<FString, int32>& InTable)
			: TrackedCommand(MoveTemp(InCommand)),
			  InboundCommandTable(InTable)
		{
			if (int32* Value = InboundCommandTable.Find(TrackedCommand))
			{
				*Value = *Value + 1;
			}
			else
			{
				int32& NewValue = InboundCommandTable.Add(TrackedCommand);
				NewValue = 1;
			}
		}

		~FScopeMultiUserReceiveCVar()
		{
			if (int32* Value = InboundCommandTable.Find(TrackedCommand))
			{
				*Value = *Value - 1;
				if (*Value == 0)
				{
					InboundCommandTable.Remove(TrackedCommand);
				}
			}
		}

		FString TrackedCommand;
		TMap<FString, int32>& InboundCommandTable;
	};

	void OnFEngineLoopInitComplete();

	void RegisterMenuItem();
	void RegisterProjectSettings() const;
	void OnConsoleObjectUnregistered(const TCHAR* InName, IConsoleObject* InConsoleObject);

	TObjectPtr<UConsoleVariablesAsset> AllocateTransientPreset(const FName DesiredName) const;
	void CreateEditingPresets();

	TSharedRef<SDockTab> SpawnMainPanelTab(const FSpawnTabArgs& Args);

	static void OpenConsoleVariablesEditor();

	static const FName ConsoleVariablesToolkitPanelTabId;

	/** Lives for as long as the module is loaded. */
	TSharedPtr<FConsoleVariablesEditorMainPanel> MainPanel;

	/** Transient preset that's being edited so we don't affect the reference asset unless we save it */
	TObjectPtr<UConsoleVariablesAsset> EditingPresetAsset = nullptr;
	/** Transient preset that tracks variables that match the search criteria */
	TObjectPtr<UConsoleVariablesAsset> EditingGlobalSearchAsset = nullptr;

	/** All tracked variables and their default, startup, and current values */
	TMap<FString, TSharedPtr<FConsoleVariablesEditorCommandInfo>> ConsoleObjectsMainReference;

	/**
	 * A map of commands invoked by Multi-user.  Each command has a corresponding reference count.
	 * When this node receives a command from another node, it will be added to this list.  This is prevent
	 * a remote cvar change creating a ping/pong cvar updates effect between nodes.
	 */
	TMap<FString, int32> CommandsReceivedFromMultiUser;

	FDelegateHandle OnConsoleObjectUnregisteredHandle;
};
