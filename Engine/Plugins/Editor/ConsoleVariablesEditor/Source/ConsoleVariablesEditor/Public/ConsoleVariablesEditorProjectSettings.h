// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConsoleVariablesEditorProjectSettings.generated.h"

UENUM(BlueprintType)
enum class EConsoleVariablesEditorRowDisplayType : uint8
{
	ShowCurrentValue,
	ShowLastEnteredValue
};

UENUM(BlueprintType)
enum class EConsoleVariablesEditorPresetImportMode : uint8
{
	/**
	 * Add the list of variables from the imported preset to the current preset, replacing the values of any overlapping
	 * variables with the values from the imported preset.
	 */
	AddToExisting,

	/**
	 * Completely replace the list of variables in the current preset, resetting them to their default values and removing
	 * them from the list before importing the new preset's variable list.
	 */
	ReplaceExisting,

	/**
	 * Use the global import setting.
	 */
	UseDefault UMETA(Hidden)
};

UCLASS(MinimalAPI, config = Engine, defaultconfig)
class UConsoleVariablesEditorProjectSettings : public UObject
{
	GENERATED_BODY()
public:
	
	UConsoleVariablesEditorProjectSettings(const FObjectInitializer& ObjectInitializer)
	{
		UncheckedRowDisplayType = EConsoleVariablesEditorRowDisplayType::ShowCurrentValue;
		PresetImportMode = EConsoleVariablesEditorPresetImportMode::AddToExisting;
		
		bAddAllChangedConsoleVariablesToCurrentPreset = true;
	}

	/**
	 *When a row is unchecked, its associated variable's value will be set to the value recorded when the plugin was loaded.
	 *The value displayed to the user can be configured with this setting, but will not affect the actual applied value.
	 *ShowCurrentValue displays the actual value currently applied to the variable.
	 *ShowLastEnteredValue displays the value that will be applied when the row is checked.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Console Variables Editor")
	EConsoleVariablesEditorRowDisplayType UncheckedRowDisplayType;

	/**
	 * When importing a console variable preset, the variables 
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Console Variables Editor")
	EConsoleVariablesEditorPresetImportMode PresetImportMode;

	/**
	 *When variables are changed outside the Console Variables Editor, this option will add the variables to the current preset.
	 *Does not apply to console commands like 'r.SetNearClipPlane' or 'stat fps'
	 */
	UPROPERTY(Config, EditAnywhere, Category="Console Variables Editor")
	bool bAddAllChangedConsoleVariablesToCurrentPreset;

	/**
	 * If bAddAllChangedConsoleVariablesToCurrentPreset is true, this list will filter out any matching variables
	 * changed outside of the Console Variables Editor so they won't be added to the current preset.
	 * Matching variables explicitly added inside the Console Variables Editor will not be filtered.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Console Variables Editor")
	TSet<FString> ChangedConsoleVariableSkipList;
};
