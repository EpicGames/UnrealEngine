// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConsoleVariablesEditorCustomConsoleInputBox.h"

#include "ConsoleVariablesEditorCommandInfo.h"
#include "ConsoleVariablesEditorLog.h"
#include "ConsoleVariablesEditorModule.h"
#include "Views/List/ConsoleVariablesEditorList.h"
#include "Views/MainPanel/SConsoleVariablesEditorMainPanel.h"

#include "Framework/Application/SlateApplication.h"
#include "OutputLogModule.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Views/Widgets/SConsoleVariablesEditorTooltipWidget.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ConsoleVariablesEditor"

void SConsoleVariablesEditorCustomConsoleInputBox::Construct(
	const FArguments& InArgs)
{
	bHideOnFocusLost = InArgs._HideOnFocusLost;
	bCommitOnFocusLost = InArgs._CommitOnFocusLost;
	bClearOnCommit = InArgs._ClearOnCommit;
	IsEnabledAttribute = InArgs._IsEnabled;
	Font = InArgs._Font;
	HintText = InArgs._HintText;
	TextAttribute = InArgs._Text;
	OnTextCommitted = InArgs._OnTextCommitted;
	OnTextChanged = InArgs._OnTextChanged;
	
	ChildSlot
	[
		SAssignNew( SuggestionBox, SMenuAnchor )
		.Method(GIsEditor ? EPopupMethod::CreateNewWindow : EPopupMethod::UseCurrentWindow)
		.Placement( MenuPlacement_BelowAnchor )
		[
			SAssignNew(InputText, SEditableTextBox)
			.IsEnabled(IsEnabledAttribute)
			.Text(TextAttribute)
			.ToolTipText_Lambda([this]()
			{
				const FString SanitizedCommand = GetSanitizedCommand(InputText->GetText().ToString());

				FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();
				const TWeakPtr<FConsoleVariablesEditorCommandInfo> MatchingCommand = ConsoleVariablesEditorModule.FindCommandInfoByName(SanitizedCommand);
				if (MatchingCommand.IsValid())
				{
					return FText::FromString(MatchingCommand.Pin()->GetHelpText());
				}

				return FText();
			})
			.Font(Font.HasValidFont() ? Font : FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Log.Normal").Font)
			.HintText(!HintText.IsEmpty() ? HintText : LOCTEXT("ConsoleCommandExecutorHintText", "Enter Console Command"))
			.OnTextChanged(this, &SConsoleVariablesEditorCustomConsoleInputBox::OnInputTextChanged)
			.OnKeyCharHandler(this, &SConsoleVariablesEditorCustomConsoleInputBox::OnKeyCharHandler)
			.OnKeyDownHandler(this, &SConsoleVariablesEditorCustomConsoleInputBox::OnKeyDownHandler)
			.OnTextCommitted_Lambda([this] (const FText& InText, const ETextCommit::Type CommitType)
			{
				// Hide widget when focus is lost
				if (CommitType == ETextCommit::OnUserMovedFocus)
				{
					// If the newly focused widget is the suggestion list, we want the suggestion list to handle what's next
					if (FSlateApplication::Get().GetUserFocusedWidget(0) != SuggestionListView &&
						FSlateApplication::Get().GetUserFocusedWidget(0) != InputText)
					{
						SuggestionBox->SetIsOpen(false);

						if (bCommitOnFocusLost)
						{
							CommitInput();
						}

						if (bHideOnFocusLost)
						{
							SetVisibility(EVisibility::Collapsed);
						}
					}
				}
			})
			.ClearKeyboardFocusOnCommit(false)
		
		]
		.MenuContent
		(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			.Padding( FMargin(2) )
			[
				SNew(SBox)
				.HeightOverride(250) // avoids flickering, ideally this would be adaptive to the content without flickering
				.MinDesiredWidth(300)
				.MaxDesiredWidth_Lambda([this] ()
				{
					// Limit the width of the suggestions list to the work area that this widget currently resides on
					const FSlateRect WidgetRect(
						GetCachedGeometry().GetAbsolutePosition(),
						GetCachedGeometry().GetAbsolutePosition() + GetCachedGeometry().GetAbsoluteSize());
					const FSlateRect WidgetWorkArea = FSlateApplication::Get().GetWorkArea(WidgetRect);
					return FMath::Max(300.0f, WidgetWorkArea.GetSize().X - 12.0f);
				})
				[
					SAssignNew(SuggestionListView, SListView< TSharedPtr<FString> >)
					.ListItemsSource(&Suggestions.SuggestionsList)
					.SelectionMode( ESelectionMode::Single )							// Ideally the mouse over would not highlight while keyboard controls the UI
					.OnGenerateRow_Lambda([this] (TSharedPtr<FString> Text, const TSharedRef<STableViewBase>& OwnerTable)
					{
						check(Text.IsValid());

						const FString SanitizedText = GetSanitizedCommand(*Text);
						const FText DisplayText = FText::FromString(SanitizedText);
						
						FConsoleVariablesEditorModule& ConsoleVariablesEditorModule = FConsoleVariablesEditorModule::Get();

						TSharedPtr<IToolTip> ToolTip;

						if (const TWeakPtr<FConsoleVariablesEditorCommandInfo> MatchingCommand =
							ConsoleVariablesEditorModule.FindCommandInfoByName(SanitizedText); MatchingCommand.IsValid())
						{
							ToolTip = SConsoleVariablesEditorTooltipWidget::MakeTooltip(
								SanitizedText,
								MatchingCommand.Pin()->GetHelpText());
						}

						return
							SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
							[
								SNew(STextBlock)
								.Text(DisplayText)
								.TextStyle(FAppStyle::Get(), "Log.Normal")
								.HighlightText(Suggestions.SuggestionsHighlight)
								.ColorAndOpacity(FSlateColor::UseForeground())
								.ToolTip(ToolTip.IsValid() ? ToolTip : nullptr)
							];
					})
					.OnSelectionChanged_Lambda([this] (TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
					{
						if(bIgnoreUIUpdate)
						{
							return;
						}
						
						Suggestions.SelectedSuggestion = Suggestions.SuggestionsList.IndexOfByPredicate([&NewValue](const TSharedPtr<FString>& InSuggestion)
						{
							return InSuggestion == NewValue;
						});

						MarkActiveSuggestion();

						// Just close the suggestion box and add the marked suggestion to the list and close up
						if( SelectInfo == ESelectInfo::OnMouseClick )
						{
							CommitInput();

							if (bHideOnFocusLost)
							{
								SetVisibility(EVisibility::Collapsed);
							}
						}
					})
				]
			]
		)
	];
}

SConsoleVariablesEditorCustomConsoleInputBox::~SConsoleVariablesEditorCustomConsoleInputBox()
{
	InputText.Reset();
	SuggestionBox.Reset();
	SuggestionListView.Reset();
}

FReply SConsoleVariablesEditorCustomConsoleInputBox::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent)
{
	if(SuggestionBox->IsOpen())
	{
		if(KeyEvent.GetKey() == EKeys::Up || KeyEvent.GetKey() == EKeys::Down)
		{
			Suggestions.StepSelectedSuggestion(KeyEvent.GetKey() == EKeys::Up ? -1 : +1);
			MarkActiveSuggestion();

			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Tab)
		{
			if (Suggestions.HasSuggestions())
			{
				if (Suggestions.HasSelectedSuggestion())
				{
					Suggestions.StepSelectedSuggestion(KeyEvent.IsShiftDown() ? -1 : +1);
				}
				else
				{
					Suggestions.SelectedSuggestion = 0;
				}
				MarkActiveSuggestion();
			}
			
			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Escape)
		{
			SuggestionBox->SetIsOpen(false);
			return FReply::Handled();
		}
	}
	else
	{
		if(KeyEvent.GetKey() == EKeys::Up)
		{
			// If the command field isn't empty we need you to have pressed Control+Up to summon the history (to make sure you're not just using caret navigation)
			const bool bShowHistory = InputText->GetText().IsEmpty() || KeyEvent.IsControlDown();
			if (bShowHistory)
			{
				TArray<FString> History;
				IConsoleManager::Get().GetConsoleHistory(TEXT(""), History);
				
				SetSuggestions(History, FText::GetEmpty());
				
				if(Suggestions.HasSuggestions())
				{
					Suggestions.StepSelectedSuggestion(-1);
					MarkActiveSuggestion();
				}
			}

			// Need to always handle this for single-line controls to avoid them invoking widget navigation
			if (bShowHistory)
			{
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

bool SConsoleVariablesEditorCustomConsoleInputBox::TakeKeyboardFocus() const
{
	check(InputText.IsValid());
	
	return FSlateApplication::Get().SetKeyboardFocus(InputText.ToSharedRef());
}

void SConsoleVariablesEditorCustomConsoleInputBox::OnInputTextChanged(const FText& InText)
{
	if(bIgnoreUIUpdate)
	{
		return;
	}
	
	const FString& InputTextStr = InText.ToString();
	if(!InputTextStr.IsEmpty())
	{
		TArray<FString> AutoCompleteList;
		
		auto OnConsoleVariable = [&AutoCompleteList](const TCHAR *Name, IConsoleObject* CVar)
		{
			if (CVar->IsEnabled())
			{	
				AutoCompleteList.Add(Name);
			}
		};

		IConsoleManager::Get().ForEachConsoleObjectThatContains(FConsoleObjectVisitor::CreateLambda(OnConsoleVariable), *InputTextStr);

		AutoCompleteList.Sort([InputTextStr](const FString& A, const FString& B)
		{ 
			if (A.StartsWith(InputTextStr))
			{
				if (!B.StartsWith(InputTextStr))
				{
					return true;
				}
			}
			else
			{
				if (B.StartsWith(InputTextStr))
				{
					return false;
				}
			}

			return A < B;

		});


		SetSuggestions(AutoCompleteList, InText);
	}
	else
	{
		ClearSuggestions();
	}

	OnTextChanged.ExecuteIfBound(InText);
}

FReply SConsoleVariablesEditorCustomConsoleInputBox::OnKeyCharHandler(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent) const
{
	// Intercept tab if used for auto-complete
	if (InCharacterEvent.GetCharacter() == '\t')
	{
		return FReply::Handled();
	}

	if (InCharacterEvent.GetModifierKeys().AnyModifiersDown() && InCharacterEvent.GetCharacter() == ' ')
	{	
		// Ignore space bar + a modifier key.  It should not type a space as this is used by other keyboard shortcuts
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SConsoleVariablesEditorCustomConsoleInputBox::OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& KeyPressed)
{
	if (KeyPressed.GetKey().GetFName() == TEXT("Enter"))
	{
		CommitInput();
	}

	return FReply::Unhandled();
}

void SConsoleVariablesEditorCustomConsoleInputBox::SetSuggestions(TArray<FString>& Elements, FText Highlight)
{
	FString SelectionText;
	if (Suggestions.HasSelectedSuggestion())
	{
		SelectionText = *Suggestions.GetSelectedSuggestion();
	}

	Suggestions.Reset();
	Suggestions.SuggestionsHighlight = Highlight;

	for(int32 i = 0; i < Elements.Num(); ++i)
	{
		Suggestions.SuggestionsList.Add(MakeShared<FString>(Elements[i]));

		if (Elements[i] == SelectionText)
		{
			Suggestions.SelectedSuggestion = i;
		}
	}
	SuggestionListView->RequestListRefresh();

	if(Suggestions.HasSuggestions())
	{
		// Ideally if the selection box is open the output window is not changing it's window title (flickers)
		SuggestionBox->SetIsOpen(true, false);
		if (Suggestions.HasSelectedSuggestion())
		{
			SuggestionListView->RequestScrollIntoView(Suggestions.GetSelectedSuggestion());
		}
		else
		{
			SuggestionListView->ScrollToTop();
		}
	}
	else
	{
		SuggestionBox->SetIsOpen(false);
	}
}

void SConsoleVariablesEditorCustomConsoleInputBox::MarkActiveSuggestion()
{
	bIgnoreUIUpdate = true;
	if (Suggestions.HasSelectedSuggestion())
	{
		TSharedPtr<FString> SelectedSuggestion = Suggestions.GetSelectedSuggestion();

		SuggestionListView->SetSelection(SelectedSuggestion);
		SuggestionListView->RequestScrollIntoView(SelectedSuggestion);	// Ideally this would only scroll if outside of the view

		const FText SelectedText = FText::FromString(*SelectedSuggestion);
		OnTextChanged.ExecuteIfBound(SelectedText);

		if (!TextAttribute.IsBound())
		{
			// Update the text input manually if it's not bound to an attribute
			InputText->SetText(SelectedText);
		}
		else
		{
			// The input seems to need focus when the suggestions box is open in order to receive updates. This
			// is only a problem if receiving updates from the bound Text attribute.
			TakeKeyboardFocus();
		}
	}
	else
	{
		SuggestionListView->ClearSelection();
	}
	bIgnoreUIUpdate = false;
}

void SConsoleVariablesEditorCustomConsoleInputBox::ClearSuggestions()
{
	SuggestionBox->SetIsOpen(false);
	Suggestions.Reset();
}

void SConsoleVariablesEditorCustomConsoleInputBox::CommitInput()
{
	const FText CommittedText = InputText->GetText();
	bIgnoreUIUpdate = true;

	if (bClearOnCommit)
	{
		OnTextChanged.ExecuteIfBound(FText::GetEmpty());

		if (!TextAttribute.IsBound())
		{
			InputText->SetText(FText::GetEmpty());
		}
	}
	else
	{
		OnTextChanged.ExecuteIfBound(CommittedText);
	}

	OnTextCommitted.ExecuteIfBound(CommittedText);

	SuggestionBox->SetIsOpen( false );
	bIgnoreUIUpdate = false;
}

FString SConsoleVariablesEditorCustomConsoleInputBox::GetSanitizedCommand(const FString& InCommand) const
{
	FString SanitizedText = InCommand;
	SanitizedText.ReplaceInline(TEXT("\r\n"), TEXT("\n"), ESearchCase::CaseSensitive);
	SanitizedText.ReplaceInline(TEXT("\r"), TEXT(" "), ESearchCase::CaseSensitive);
	SanitizedText.ReplaceInline(TEXT("\n"), TEXT(" "), ESearchCase::CaseSensitive);

	return SanitizedText;
}

#undef LOCTEXT_NAMESPACE
