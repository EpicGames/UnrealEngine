// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Views/SListView.h"

#define UE_API CONSOLEVARIABLESEDITOR_API

class SEditableTextBox;

class SConsoleVariablesEditorCustomConsoleInputBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnTextCommitted, const FText&);
	DECLARE_DELEGATE_OneParam(FOnTextChanged, const FText&);

	SLATE_BEGIN_ARGS(SConsoleVariablesEditorCustomConsoleInputBox)
		: _CommitOnFocusLost(false)
	{}
	
		/** If true, hide the input box when focus is lost. */
		SLATE_ARGUMENT(bool, HideOnFocusLost)

		/** If true, commit the input box's text when the input box loses focus (calls the OnTextCommitted event). */
		SLATE_ARGUMENT(bool, CommitOnFocusLost)

		/** If true, clear the input box when text is committed. */
		SLATE_ARGUMENT(bool, ClearOnCommit)

		/** The enable state of the input box. */
		SLATE_ATTRIBUTE(bool, IsEnabled)

		/** Font override for the input box. */
		SLATE_ARGUMENT(FSlateFontInfo, Font)

		/** Hint text override for the input box. */
		SLATE_ARGUMENT(FText, HintText)

		/** The text to show in the input box. */
		SLATE_ATTRIBUTE(FText, Text)

		/** Called when text in the input box is committed. */
		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)

		/** Called when text in the input box has changed. */
		SLATE_EVENT(FOnTextChanged, OnTextChanged)
	SLATE_END_ARGS()
	
	struct FSuggestions
	{
		FSuggestions()
			: SelectedSuggestion(INDEX_NONE)
		{
		}

		void Reset()
		{
			SelectedSuggestion = INDEX_NONE;
			SuggestionsList.Reset();
			SuggestionsHighlight = FText::GetEmpty();
		}

		bool HasSuggestions() const
		{
			return SuggestionsList.Num() > 0;
		}

		bool HasSelectedSuggestion() const
		{
			return SuggestionsList.IsValidIndex(SelectedSuggestion);
		}

		void StepSelectedSuggestion(const int32 Step)
		{
			SelectedSuggestion += Step;
			if (SelectedSuggestion < 0)
			{
				SelectedSuggestion = SuggestionsList.Num() - 1;
			}
			else if (SelectedSuggestion >= SuggestionsList.Num())
			{
				SelectedSuggestion = 0;
			}
		}

		TSharedPtr<FString> GetSelectedSuggestion() const
		{
			return SuggestionsList.IsValidIndex(SelectedSuggestion) ? SuggestionsList[SelectedSuggestion] : nullptr;
		}

		/** INDEX_NONE if not set, otherwise index into SuggestionsList */
		int32 SelectedSuggestion;

		/** All log messages stored in this widget for the list view */
		TArray<TSharedPtr<FString>> SuggestionsList;

		/** Highlight text to use for the suggestions list */
		FText SuggestionsHighlight;
	};

	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual ~SConsoleVariablesEditorCustomConsoleInputBox() override;

	//~ Begin SWidget Interface
	UE_API virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent) override;
	//~ End SWidget Interface

	UE_API bool TakeKeyboardFocus() const;

	UE_API void OnInputTextChanged(const FText& InText);

	UE_API FReply OnKeyCharHandler(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent) const;

	UE_API FReply OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& KeyPressed);
	
	UE_API void SetSuggestions(TArray<FString>& Elements, FText Highlight);

	UE_API void MarkActiveSuggestion();

	UE_API void ClearSuggestions();
	
	UE_API void CommitInput();

private:
	UE_API FString GetSanitizedCommand(const FString& InCommand) const;

private:
	
	/** A reference to the actual text box inside ConsoleInput */
	TSharedPtr<SEditableTextBox> InputText;

	/** history / auto completion elements */
	TSharedPtr<SMenuAnchor> SuggestionBox;

	/** The list view for showing all log messages. Should be replaced by a full text editor */
	TSharedPtr<SListView<TSharedPtr<FString>>> SuggestionListView;

	/** Active list of suggestions */
	FSuggestions Suggestions;

	/** to prevent recursive calls in UI callback */
	bool bIgnoreUIUpdate = false;

	bool bHideOnFocusLost = true;
	bool bCommitOnFocusLost = false;
	bool bClearOnCommit = true;
	TAttribute<bool> IsEnabledAttribute;
	FSlateFontInfo Font;
	FText HintText;
	TAttribute<FText> TextAttribute;
	FOnTextCommitted OnTextCommitted;
	FOnTextChanged OnTextChanged;
};

#undef UE_API
