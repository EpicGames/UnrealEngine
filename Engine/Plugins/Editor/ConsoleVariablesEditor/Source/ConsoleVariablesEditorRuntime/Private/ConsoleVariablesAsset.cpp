// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleVariablesAsset.h"
#include "Templates/Tuple.h"

#include "ConsoleVariablesEditorRuntimeLog.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConsoleVariablesAsset)

void UConsoleVariablesAsset::SetVariableCollectionDescription(const FString& InVariableCollectionDescription)
{
	VariableCollectionDescription = InVariableCollectionDescription;
}

TArray<FString> UConsoleVariablesAsset::GetSavedCommandsAsStringArray(bool bOnlyIncludeChecked) const
{
	TArray<FString> ReturnValue;
	
	// Make a copy because this array can change during iteration
	TArray<FConsoleVariablesEditorAssetSaveData> SavedCommandsLocal = SavedCommands; 
	for (int32 CommandIndex = 0; CommandIndex < SavedCommandsLocal.Num(); CommandIndex++)
	{
		const FConsoleVariablesEditorAssetSaveData& Command = SavedCommandsLocal[CommandIndex];
		
		if (bOnlyIncludeChecked && Command.CheckedState != ECheckBoxState::Checked)
		{
			continue;
		}
		
		ReturnValue.Add(FString::Printf(TEXT("%s %s"), *Command.CommandName, *Command.CommandValueAsString));
	}

	return ReturnValue;
}

FString UConsoleVariablesAsset::GetSavedCommandsAsCommaSeparatedString(bool bOnlyIncludeChecked) const
{
	return FString::Join(GetSavedCommandsAsStringArray(bOnlyIncludeChecked), TEXT(","));
}

void UConsoleVariablesAsset::GetConsoleVariablesForTrack(bool bOnlyIncludeChecked, TArray<TTuple<FString, FString>>& OutVariables)
{
	for (const FConsoleVariablesEditorAssetSaveData& Command : SavedCommands)
	{
		if (bOnlyIncludeChecked && Command.CheckedState != ECheckBoxState::Checked)
		{
			continue;
		}
		
		OutVariables.Add(MakeTuple(Command.CommandName, Command.CommandValueAsString));
	}
}

bool UConsoleVariablesAsset::IsConsoleVariableEnabled(const FString& ConsoleVariableName)
{
	for (const FConsoleVariablesEditorAssetSaveData& Command : SavedCommands)
	{
		if (Command.CommandName == ConsoleVariableName)
		{
			return Command.CheckedState == ECheckBoxState::Checked;
		}
	}

	return false;
}

void UConsoleVariablesAsset::ExecuteSavedCommands(UObject* WorldContextObject, bool bOnlyIncludeChecked) const
{
	if (const TObjectPtr<UWorld> World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		TArray<FString> Commands = GetSavedCommandsAsStringArray(bOnlyIncludeChecked);
	
		for (const FString& Command : Commands)
		{
			GEngine->Exec(World, *Command);
		}
	}
	else
	{
		UE_LOGF(LogConsoleVariablesEditorRuntime, Error, "%s: Could not get world from WorldContextObject", __FUNCTION__);
	}
}

void UConsoleVariablesAsset::ReplaceSavedCommands(const TArray<FConsoleVariablesEditorAssetSaveData>& Replacement)
{
	SavedCommands = Replacement;
}

bool UConsoleVariablesAsset::FindSavedDataByCommandString(
	const FString& InCommandString, FConsoleVariablesEditorAssetSaveData& OutValue, const ESearchCase::Type SearchCase) const
{
	if (!SavedCommands.IsEmpty())
	{
		const FStringView InCommandTrimmed = FStringView(InCommandString).TrimStartAndEnd();
		for (const FConsoleVariablesEditorAssetSaveData& Command : SavedCommands)
		{
			if (FStringView(Command.CommandName).TrimStartAndEnd().Equals(InCommandTrimmed, SearchCase))
			{
				OutValue = Command;
				return true;
			}
		}
	}
	
	return false;
}

void UConsoleVariablesAsset::AddOrSetConsoleObjectSavedData(const FConsoleVariablesEditorAssetSaveData& InData)
{
	RemoveConsoleVariable(InData.CommandName);
	SavedCommands.Add(InData);
	
	// Make a copy because this array can change during iteration
	TArray<FConsoleVariablesEditorAssetSaveData> SavedCommandsLocal = SavedCommands; 
	for (int32 CommandIndex = 0; CommandIndex < SavedCommandsLocal.Num(); CommandIndex++)
	{
		FConsoleVariablesEditorAssetSaveData& SavedCommand = SavedCommandsLocal[CommandIndex];
		UE_LOGF(LogConsoleVariablesEditorRuntime, VeryVerbose, "%s: Command named '%ls' at Index %i",
			__FUNCTION__, *SavedCommand.CommandName, CommandIndex);
	}
}

bool UConsoleVariablesAsset::RemoveConsoleVariable(const FString& InCommandString)
{
	FConsoleVariablesEditorAssetSaveData ExistingData;
	int32 RemoveCount = 0;
	while (FindSavedDataByCommandString(InCommandString, ExistingData, ESearchCase::IgnoreCase))
	{
		UE_LOGF(LogConsoleVariablesEditorRuntime, VeryVerbose, "%s: Removing %ls from editable asset",
			__FUNCTION__, *InCommandString);
		
		if (SavedCommands.Remove(ExistingData) > 0)
		{
			RemoveCount++;
		}
	}
	
	return RemoveCount > 0;
}

void UConsoleVariablesAsset::CopyFrom(const UConsoleVariablesAsset* InAssetToCopy)
{
	VariableCollectionDescription = InAssetToCopy->GetVariableCollectionDescription();
	SavedCommands = InAssetToCopy->GetSavedCommands();
}

void UConsoleVariablesAsset::AddFrom(const UConsoleVariablesAsset* InAssetToCopy)
{
	for (const FConsoleVariablesEditorAssetSaveData& NewCommand : InAssetToCopy->GetSavedCommands())
	{
		FConsoleVariablesEditorAssetSaveData* ExistingCommand = SavedCommands.FindByPredicate(
			[NewCommand](const FConsoleVariablesEditorAssetSaveData& ExistingCommand)
			{
				return ExistingCommand.CommandName.Compare(NewCommand.CommandName) == 0;
			}
		);

		if (ExistingCommand)
		{
			ExistingCommand->CommandValueAsString = NewCommand.CommandValueAsString;
			ExistingCommand->CheckedState = ECheckBoxState::Checked;
		}
		else
		{
			SavedCommands.Add(NewCommand);
		}
	}
}
