// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Styling/SlateTypes.h"
#include "Sections/MovieSceneConsoleVariableTrackInterface.h"

#include "ConsoleVariablesAsset.generated.h"

#define UE_API CONSOLEVARIABLESEDITORRUNTIME_API

/** Data that will be serialized with this asset */
USTRUCT(BlueprintType)
struct FConsoleVariablesEditorAssetSaveData
{
	GENERATED_BODY()

	FORCEINLINE bool operator==(const FConsoleVariablesEditorAssetSaveData& Comparator) const
	{
		return CommandName.Equals(Comparator.CommandName);
	}
	
	UPROPERTY()
	FString CommandName;

	UPROPERTY()
	FString CommandValueAsString;

	UPROPERTY()
	/** If Undetermined, we can assume this data was not previously saved */
	ECheckBoxState CheckedState = ECheckBoxState::Undetermined;
};

/** An asset used to track collections of console variables that can be recalled and edited using the Console Variables Editor. */
UCLASS(MinimalAPI, BlueprintType)
class UConsoleVariablesAsset : public UObject, public IMovieSceneConsoleVariableTrackInterface
{
	GENERATED_BODY()
public:

	/** Sets a description for this variable collection. */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Editor")
	UE_API void SetVariableCollectionDescription(const FString& InVariableCollectionDescription);
	
	UFUNCTION(BlueprintPure, Category = "Console Variables Editor")
	FString GetVariableCollectionDescription() const
	{
		return VariableCollectionDescription;
	}

	/** Returns the saved list of console variable information such as the variable name, the type and the value of the variable at the time the asset was saved. */
	UFUNCTION(BlueprintPure, Category="Console Variables Asset")
	const TArray<FConsoleVariablesEditorAssetSaveData>& GetSavedCommands() const
	{
		return SavedCommands;
	}

	/** Returns the saved list of console variables as an array of FString.
	 * @param bOnlyIncludeChecked If true, only commands and variables with a Checked checkstate in the Console Variables Editor UI will be included. Otherwise, all will be included. 
	 */
	UFUNCTION(BlueprintPure, Category="Console Variables Asset")
	UE_API TArray<FString> GetSavedCommandsAsStringArray(bool bOnlyIncludeChecked = false) const;

	/** Returns the saved list of console variables as a concatenated comma-separated string. Useful for passing commands and variables to a command line.
	 * @param bOnlyIncludeChecked If true, only commands and variables with a Checked checkstate in the Console Variables Editor UI will be included. Otherwise, all will be included. 
	 */
	UFUNCTION(BlueprintPure, Category="Console Variables Asset")
	UE_API FString GetSavedCommandsAsCommaSeparatedString(bool bOnlyIncludeChecked = false) const;

	/** Executes all saved commands in this asset, optionally only including those with a Checked checkstate in the UI.
	 * @param bOnlyIncludeChecked If true, only commands and variables with a Checked checkstate in the Console Variables Editor UI will be included. Otherwise, all will be included. 
	 */
	UFUNCTION(BlueprintCallable, Category="Console Variables Asset", meta=(WorldContext="WorldContextObject"))
	UE_API void ExecuteSavedCommands(UObject* WorldContextObject, bool bOnlyIncludeChecked = false) const;

	/** Completely replaces the saved data with new saved data */
	UFUNCTION(BlueprintCallable, Category="Console Variables Asset")
	UE_API void ReplaceSavedCommands(const TArray<FConsoleVariablesEditorAssetSaveData>& Replacement);

	/** Returns how many console variables are serialized in this asset */
	UFUNCTION(BlueprintPure, Category="Console Variables Asset")
	int32 GetSavedCommandsCount() const
	{
		return GetSavedCommands().Num();
	}

	/** Outputs the FConsoleVariablesEditorAssetSaveData matching InCommand. Returns whether a match was found. Case sensitive. */
	UFUNCTION(BlueprintCallable, Category="Console Variables Asset")
	UE_API bool FindSavedDataByCommandString(
		const FString& InCommandString, FConsoleVariablesEditorAssetSaveData& OutValue, const ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const;

	/** Set the value of a saved console variable if the name matches; add a new console variable to the list if a match is not found. */
	UFUNCTION(BlueprintCallable, Category="Console Variables Asset")
	UE_API void AddOrSetConsoleObjectSavedData(const FConsoleVariablesEditorAssetSaveData& InData);

	/** Returns true if the element was found and successfully removed. */
	UFUNCTION(BlueprintCallable, Category="Console Variables Asset")
	UE_API bool RemoveConsoleVariable(const FString& InCommandString);

	/** Copy list of variables from input asset to this asset */
	UFUNCTION(BlueprintCallable, Category="Console Variables Asset")
	UE_API void CopyFrom(const UConsoleVariablesAsset* InAssetToCopy);

	/** Add list of variables from input asset to this asset's list */
	UFUNCTION(BlueprintCallable, Category = "Console Variables Asset")
	UE_API void AddFrom(const UConsoleVariablesAsset* InAssetToCopy);
	
private:

	//~ IMovieSceneConsoleVariableTrackInterface overrides
	UE_API virtual void GetConsoleVariablesForTrack(bool bOnlyIncludeChecked, TArray<TPair<FString, FString>>& OutVariables) override;
	UE_API virtual bool IsConsoleVariableEnabled(const FString& ConsoleVariableName) override;

private:

	/* User defined description of the variable collection */
	UPROPERTY(AssetRegistrySearchable, BlueprintGetter = "GetVariableCollectionDescription", EditAnywhere, Category = "Console Variables Editor")
	FString VariableCollectionDescription;

	/** A saved list of console variable information such as the variable name, the type and the value of the variable at the time the asset was saved. */
	UPROPERTY()
	TArray<FConsoleVariablesEditorAssetSaveData> SavedCommands;
};

#undef UE_API
